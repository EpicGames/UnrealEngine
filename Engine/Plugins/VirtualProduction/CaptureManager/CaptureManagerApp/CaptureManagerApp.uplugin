{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0.0",
	"FriendlyName": "Capture Manager Application",
	"Description": "The Capture Manager allows control and monitoring of the capture device, obtains and transcodes the data from the devices and upload the data for import to the UE",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Hidden": true,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "CaptureManagerEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerUnrealEndpoint",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CaptureManagerSettings",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkCapabilities",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "IngestLiveLinkDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LiveLinkHub",
			"Enabled": true
		},
		{
			"Name": "CaptureManagerCore",
			"Enabled": true
		},
		{
			"Name": "CaptureData",
			"Enabled": true
		},
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		},
		{
			"Name": "ElectraPlayer",
			"Enabled": true
		},
		{
			"Name": "MJPEGDecoderElectra",
			"Enabled": true
		},
		{
			"Name": "MediaFrameworkUtilities",
			"Enabled": true
		}
	]
}