[CoreRedirects]
+EnumRedirects=(OldName="/Script/LiveLinkCapabilities.EIngestCapability_ImagePixelFormat", NewName="/Script/CaptureDataConverter.ECaptureManagerPixelFormat")
+EnumRedirects=(OldName="/Script/LiveLinkCapabilities.EIngestCapability_ImageRotation", NewName="/Script/CaptureDataConverter.ECaptureManagerRotation")

[/Script/LiveLinkHub.LiveLinkHubSettings]
+ProjectSettingsToDisplay=Capture Manager