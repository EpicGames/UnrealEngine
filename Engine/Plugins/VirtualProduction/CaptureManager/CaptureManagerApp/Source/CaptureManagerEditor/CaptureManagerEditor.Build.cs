// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CaptureManagerEditor : ModuleRules
{
	public CaptureManagerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"EditorStyle"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"InputCore",
			"LiveLinkDevice",
			"LiveLinkEditor",
			"LiveLinkCapabilities",
			"LiveLinkHub",
			"Slate",
			"SlateCore",
			"ToolWidgets",
			"WorkspaceMenuStructure",
			"ContentBrowser",
			"UnrealEd",
			"Engine",
			"CaptureUtils",
			"CaptureManagerTakeMetadata",
			"CaptureManagerStyle",
			"OutputLog",			
			"CaptureDataCore",
			"CaptureManagerSettings",
			"ImageCore",
			"NamingTokens",
			"Projects",
			"Settings",
			"CaptureManagerUnrealEndpoint",
			"CaptureDataConverter"
		});
	}
}
