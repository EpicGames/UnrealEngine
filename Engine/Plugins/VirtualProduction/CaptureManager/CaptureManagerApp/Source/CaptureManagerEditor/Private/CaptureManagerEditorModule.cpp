// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerEditorModule.h"

#include "Features/IModularFeatures.h"
#include "LiveLinkHubApplicationBase.h"
#include "UI/LiveLinkHubCaptureManagerMode.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UI/TakeThumbnailRenderer.h"
#include "UI/TakeVirtualAsset.h"

#include "Styling/SlateStyleRegistry.h"
#include "UI/IngestJobSettingsCustomization.h"

#include "IngestManagement/UIngestJobSettings.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FCaptureManagerEditorModule, CaptureManagerEditor)

void FCaptureManagerEditorModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(ILiveLinkHubApplicationModeFactory::ModularFeatureName, this);
	UThumbnailManager::Get().RegisterCustomRenderer(UTakeVirtualAsset::StaticClass(), UTakeThumbnailRenderer::StaticClass());

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout(
		UIngestJobSettings::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIngestJobSettingsCustomization::MakeInstance)
	);
}

void FCaptureManagerEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UThumbnailManager::Get().UnregisterCustomRenderer(UTakeThumbnailRenderer::StaticClass());

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomClassLayout(UIngestJobSettings::StaticClass()->GetFName());
	}
}

/** Instantiate an application mode so LiveLinkHub can register it and display it in its Layout Selector. */
TSharedRef<FLiveLinkHubApplicationMode> FCaptureManagerEditorModule::CreateLiveLinkHubAppMode(TSharedPtr<FLiveLinkHubApplicationBase> InApp)
{
	return MakeShared<FLiveLinkHubCaptureManagerMode>(InApp);
}
