// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestJob.h"

namespace UE::CaptureManager
{

FIngestJob::FIngestJob(
	FGuid InCaptureDeviceId,
	FTakeId InTakeId,
	FTakeMetadata InTakeMetadata,
	EIngestCapability_ProcessConfig InPipelineConfig,
	FSettings InSettings
) :
	JobGuid(FGuid::NewGuid()),
	CaptureDeviceId(InCaptureDeviceId),
	TakeId(InTakeId),
	TakeMetadata(MoveTemp(InTakeMetadata)),
	PipelineConfig(InPipelineConfig),
	Settings(MoveTemp(InSettings))
{
}

const FGuid& FIngestJob::GetGuid() const
{
	return JobGuid;
}

FGuid FIngestJob::GetCaptureDeviceId() const
{
	return CaptureDeviceId;
}

FTakeId FIngestJob::GetTakeId() const
{
	return TakeId;
}

const FTakeMetadata& FIngestJob::GetTakeMetadata() const
{
	return TakeMetadata;
}

FIngestJob::FProcessingState FIngestJob::GetProcessingState() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ProcessingState;
}

double FIngestJob::GetProgress() const
{
	return Progress;
}

const FIngestJob::FSettings& FIngestJob::GetSettings() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return Settings;
}

EIngestCapability_ProcessConfig FIngestJob::GetPipelineConfig() const
{
	return PipelineConfig;
}

void FIngestJob::SetProcessingState(FIngestJob::FProcessingState InProcessingState)
{
	FScopeLock ScopeLock(&CriticalSection);
	ProcessingState = MoveTemp(InProcessingState);
}

void FIngestJob::SetProgress(const double InProgress)
{
	Progress = InProgress;
}

void FIngestJob::SetSettings(FSettings InSettings)
{
	FScopeLock ScopeLock(&CriticalSection);
	Settings = MoveTemp(InSettings);
}

}