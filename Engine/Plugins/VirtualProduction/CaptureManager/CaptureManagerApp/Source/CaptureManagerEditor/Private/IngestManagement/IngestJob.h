// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "Ingest/IngestCapability_ProcessHandle.h"

#include "UIngestJobSettings.h"

namespace UE::CaptureManager
{

class FIngestJob
{
public:
	enum class EProcessingState : uint32
	{
		Pending = 1,
		Running = 2,
		Complete = 4,
		Aborted = 8
	};

	struct FProcessingState
	{
		EProcessingState State;
		FText Message;
	};

	struct FSettings
	{
		struct FVideoSettings
		{
			::EOutputImageFormat Format;
			FString FileNamePrefix;
			::EImagePixelFormat ImagePixelFormat;
			::EImageRotation ImageRotation;
		};

		struct FAudioSettings
		{
			::EAudioFormat Format;
			FString FileNamePrefix;
		};

		FString WorkingDirectory;
		FString DownloadFolder;
		FVideoSettings VideoSettings;
		FAudioSettings AudioSettings;
		FString UploadHostName;
	};

	FIngestJob(
		FGuid InCaptureDeviceId,
		FTakeId InTakeId,
		FTakeMetadata InTakeMetadata,
		EIngestCapability_ProcessConfig InPipelineConfig,
		FSettings InSettings
	);

	const FGuid& GetGuid() const;
	FGuid GetCaptureDeviceId() const;
	FTakeId GetTakeId() const;
	const FTakeMetadata& GetTakeMetadata() const;
	EIngestCapability_ProcessConfig GetPipelineConfig() const;
	const FSettings& GetSettings() const;
	FProcessingState GetProcessingState() const;
	double GetProgress() const;

	void SetSettings(FSettings Settings);
	void SetProcessingState(FProcessingState ProcessingState);
	void SetProgress(double Progress);

private:
	const FGuid JobGuid;
	const FGuid CaptureDeviceId;
	const FTakeId TakeId;
	const FTakeMetadata TakeMetadata;
	const EIngestCapability_ProcessConfig PipelineConfig;

	mutable FCriticalSection CriticalSection;

	FSettings Settings;

	static_assert(std::atomic<double>::is_always_lock_free);
	std::atomic<double> Progress = 0.0;

	FProcessingState ProcessingState = FProcessingState{ EProcessingState::Pending, FText::FromString("Pending...") };

	FText ProcessingStateMessage;
};

}