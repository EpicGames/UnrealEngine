// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestJobExecutor.h"

#include "IngestJobQueue.h"

#include "Engine/Engine.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkDevice.h"
#include "Settings/CaptureManagerSettings.h"

#include "Async/HelperFunctions.h"

#include "Editor.h"
#include "GlobalNamingTokens.h"
#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"
#include "Settings/CaptureManagerTemplateTokens.h"

#define LOCTEXT_NAMESPACE "IngestJobExecutor"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerIngestJobExecutor, Log, All);

namespace UE::CaptureManager
{

static ECaptureManagerPixelFormat ConvertImagePixelFormat(::EImagePixelFormat InImagePixelFormat)
{
	switch (InImagePixelFormat)
	{
		case ::EImagePixelFormat::U8_Mono:
			return ECaptureManagerPixelFormat::U8_Mono;
		case ::EImagePixelFormat::U8_BGRA:
		default:
			return ECaptureManagerPixelFormat::U8_BGRA;
	}
}

static ECaptureManagerRotation ConvertImageRotation(::EImageRotation InImageRotation)
{
	switch (InImageRotation)
	{
		case ::EImageRotation::CW_90:
			return ECaptureManagerRotation::CW_90;
		case ::EImageRotation::CW_180:
			return ECaptureManagerRotation::CW_180;
		case ::EImageRotation::CW_270:
			return ECaptureManagerRotation::CW_270;
		case ::EImageRotation::None:
		default:
			return ECaptureManagerRotation::None;
	}
}

TSharedRef<FIngestJobExecutor> FIngestJobExecutor::Create(
	FString InExecutorName,
	TSharedRef<FIngestJobQueue> InProcessingQueue,
	FOnComplete InOnComplete,
	FJobProcessingStateChanged InJobProcessingStateChanged
)
{
	return MakeShared<FIngestJobExecutor>(
		FPrivateToken(),
		MoveTemp(InExecutorName),
		MoveTemp(InProcessingQueue),
		MoveTemp(InOnComplete),
		MoveTemp(InJobProcessingStateChanged)
	);
}

FIngestJobExecutor::FIngestJobExecutor(
	FPrivateToken InPrivateToken,
	FString InExecutorName,
	TSharedRef<FIngestJobQueue> InProcessingQueue,
	FOnComplete InOnComplete,
	FJobProcessingStateChanged InJobProcessingStateChanged
) :
	ExecutorName(MoveTemp(InExecutorName)),
	ProcessingQueue(MoveTemp(InProcessingQueue)),
	Complete(MoveTemp(InOnComplete)),
	JobProcessingStateChanged(MoveTemp(InJobProcessingStateChanged)),
	bIsRunning(false),
	bStopRequested(false),
	Thread(TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, *ExecutorName)))
{
}

FIngestJobExecutor::~FIngestJobExecutor()
{
	if (Thread)
	{
		Thread->Kill(true);
	}
}

bool FIngestJobExecutor::Init()
{
	bIsRunning = true;
	return true;
}

uint32 FIngestJobExecutor::Run()
{
	while (!bStopRequested)
	{
		TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
		TFuture<void> Future = Promise->GetFuture();

		TSharedPtr<FIngestJob> Job = ProcessingQueue->ClaimFirstPending(FIngestJob::EProcessingState::Running);

		if (Job)
		{
			FIngestJob::EProcessingState State = Job->GetProcessingState().State;
			check(State != FIngestJob::EProcessingState::Pending);
			JobProcessingStateChanged.ExecuteIfBound(Job->GetGuid(), State);

			FScopeLock Lock(&CriticalSection);

			// RunIngest MUST set the promise for us to terminate cleanly, so we pass shared references to keep things alive in
			// the various callbacks inside RunIngest, just to help ensure this. Weak ptrs or references etc. would open up
			// possibilities (every time you try to .Pin()) for that not to be the case. Since these callbacks are a part of
			// the process handle inside the context object (which is set as a member of this class), everything should be well
			// within our control.

			// Note: This may be a blocking call depending on how the capture device has been implemented. If so, this 
			// may take a while to stop (no way to cancel in that case).
			TUniquePtr<FIngestJobExecutor::FRunIngestContext> RunIngestContext = RunIngest(Job.ToSharedRef(), Promise);
			
			// Check the context in case the device has been removed during the RunIngest call
			if (RunIngestContext)
			{
				RunIngestProcessContext = MoveTemp(RunIngestContext);

				// There's a period between RunIngest and the assignment of the member context, during which Stop() may have
				// been called. If so, immediately terminate (as Cancel() will not be called otherwise).
				if (bStopRequested && !Future.IsReady() && RunIngestProcessContext)
				{
					ILiveLinkDeviceCapability_Ingest::Execute_CancelIngestProcess(RunIngestProcessContext->Device.Get(), RunIngestProcessContext->ProcessHandle.Get());
				}
			}
		}
		else
		{
			// Job queue depleted, stop the executor
			Promise->SetValue();
			bStopRequested = true;
		}

		Future.Wait();

		{
			FScopeLock Lock(&CriticalSection);

			if (RunIngestProcessContext)
			{
				RunIngestProcessContext = nullptr;
			}
		}

		if (!bStopRequested)
		{
			FPlatformProcess::Sleep(1.0);
		}
	}

	return 0;
}

void FIngestJobExecutor::Stop()
{
	bStopRequested = true;

	FScopeLock Lock(&CriticalSection);

	if (RunIngestProcessContext.IsValid())
	{
		ILiveLinkDeviceCapability_Ingest::Execute_CancelIngestProcess(RunIngestProcessContext->Device.Get(), RunIngestProcessContext->ProcessHandle.Get());
	}
}

void FIngestJobExecutor::CancelForDevice(const FGuid& InDeviceId)
{
	FScopeLock Lock(&CriticalSection);

	if (RunIngestProcessContext.IsValid() && RunIngestProcessContext->Device.IsValid() && RunIngestProcessContext->Device->GetDeviceId() == InDeviceId)
	{
		ILiveLinkDeviceCapability_Ingest::Execute_CancelIngestProcess(RunIngestProcessContext->Device.Get(), RunIngestProcessContext->ProcessHandle.Get());
	}
}

void FIngestJobExecutor::Exit()
{
	bIsRunning = false;
	Complete.ExecuteIfBound();
}

TUniquePtr<FIngestJobExecutor::FRunIngestContext> FIngestJobExecutor::RunIngest(TSharedRef<FIngestJob> InJob, TSharedRef<TPromise<void>> InPromise)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FIngestJobExecutor_RunIngest);

	if (!GEditor)
	{
		FText Message = LOCTEXT("IngestProcessStateMessage_GeditorUnavailable", "Ingest failed, GEditor unavailble");
		UE_LOGF(
			LogCaptureManagerIngestJobExecutor,
			Error,
			"[%ls] %ls", *ExecutorName, *Message.ToString()
		);

		// Set the processing state to anything other than Pending, to prevent other executors picking up the same bad job.
		InJob->SetProcessingState(FIngestJob::FProcessingState{ FIngestJob::EProcessingState::Aborted, MoveTemp(Message) });
		InPromise->SetValue();
		return nullptr;
	}

	ULiveLinkDeviceSubsystem* DeviceSubsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	check(DeviceSubsystem);

	const TObjectPtr<ULiveLinkDevice>* DevicePtr = DeviceSubsystem->GetDeviceMap().Find(InJob->GetCaptureDeviceId());

	if (!DevicePtr)
	{
		FText Message = LOCTEXT("IngestProcessStateMessage_DeviceDoesNotExist", "Ingest failed, Capture Device does not exist");
		UE_LOGF(
			LogCaptureManagerIngestJobExecutor,
			Error,
			"[%ls] %ls", *ExecutorName, *Message.ToString()
		);

		// Set the processing state to anything other than Pending, to prevent other executors picking up the same bad job.
		InJob->SetProcessingState(FIngestJob::FProcessingState{ FIngestJob::EProcessingState::Aborted, MoveTemp(Message) });
		InPromise->SetValue();
		return nullptr;
	}

	TObjectPtr<ULiveLinkDevice> Device = *DevicePtr;
	check(Device);
	const FText DeviceName = Device->GetDisplayName();

	UIngestCapability_Options* IngestOptions = NewObject<UIngestCapability_Options>();

	// Naming tokens subsystem consults asset registry so need to run on the game thread
	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
	check(NamingTokensSubsystem);

	NamingTokensSubsystem->SetCacheEnabled(false);

	FNamingTokenFilterArgs GeneralNamingTokenArgs;
	const TObjectPtr<const UCaptureManagerGeneralTokens> Tokens = GetDefault<UCaptureManagerSettings>()->GetGeneralNamingTokens();
	check(Tokens);

	GeneralNamingTokenArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	GeneralNamingTokenArgs.bNativeOnly = true;

	const FIngestJob::FSettings Settings = InJob->GetSettings();
	// If everything is working correctly the upload host name should never be empty.
	check(!Settings.UploadHostName.IsEmpty());

	FStringFormatNamedArguments GeneralArgs;
	{
		using namespace UE::CaptureManager;
		GeneralArgs.Add(Tokens->GetToken(FString(GeneralTokens::IdKey)).Name, InJob->GetTakeMetadata().UniqueId);
		GeneralArgs.Add(Tokens->GetToken(FString(GeneralTokens::DeviceKey)).Name, DeviceName.ToString());
		GeneralArgs.Add(Tokens->GetToken(FString(GeneralTokens::SlateKey)).Name, InJob->GetTakeMetadata().Slate);
		GeneralArgs.Add(Tokens->GetToken(FString(GeneralTokens::TakeKey)).Name, InJob->GetTakeMetadata().TakeNumber);
	}

	FString WorkingDirectory = FString::Format(*Settings.WorkingDirectory, GeneralArgs);
	FNamingTokenResultData WorkingDirectoryResult = NamingTokensSubsystem->EvaluateTokenString(WorkingDirectory, GeneralNamingTokenArgs);
	IngestOptions->WorkingDirectory = WorkingDirectoryResult.EvaluatedText.ToString();

	FString DownloadDirectory = FString::Format(*Settings.DownloadFolder, GeneralArgs);
	FNamingTokenResultData DownloadDirectoryResult = NamingTokensSubsystem->EvaluateTokenString(DownloadDirectory, GeneralNamingTokenArgs);
	IngestOptions->DownloadDirectory = DownloadDirectoryResult.EvaluatedText.ToString();

	FString FileNamePrefix = FString::Format(*Settings.VideoSettings.FileNamePrefix, GeneralArgs);
	FNamingTokenResultData VideoPrefixResult = NamingTokensSubsystem->EvaluateTokenString(FileNamePrefix, GeneralNamingTokenArgs);
	FIngestCapability_VideoOptions VideoOptions;
	VideoOptions.FileNamePrefix = VideoPrefixResult.EvaluatedText.ToString();
	VideoOptions.Format = StaticEnum<EOutputImageFormat>()->GetDisplayNameTextByValue(static_cast<std::underlying_type_t<EOutputImageFormat>>(Settings.VideoSettings.Format)).ToString();
	VideoOptions.PixelFormat = ConvertImagePixelFormat(Settings.VideoSettings.ImagePixelFormat);
	VideoOptions.Rotation = ConvertImageRotation(Settings.VideoSettings.ImageRotation);
	IngestOptions->Video = VideoOptions;

	FString AudioPrefix = FString::Format(*Settings.AudioSettings.FileNamePrefix, GeneralArgs);
	FNamingTokenResultData AudioPrefixResult = NamingTokensSubsystem->EvaluateTokenString(AudioPrefix, GeneralNamingTokenArgs);
	FIngestCapability_AudioOptions AudioOptions;
	AudioOptions.FileNamePrefix = AudioPrefixResult.EvaluatedText.ToString();
	AudioOptions.Format = StaticEnum<EAudioFormat>()->GetDisplayNameTextByValue(static_cast<std::underlying_type_t<EAudioFormat>>(Settings.AudioSettings.Format)).ToString();
	IngestOptions->Audio = AudioOptions;

	IngestOptions->UploadHostName = Settings.UploadHostName;

	if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		FText Message = FText::Format(LOCTEXT("IngestProcessStateMessage_DeviceDoesNotSupportIngestCapability", "Ingest failed, Capture Device {0} does not support the ingest capability"), FText::FromString(Device->GetName()));
		UE_LOGF(
			LogCaptureManagerIngestJobExecutor,
			Error,
			"[%ls] %ls", *ExecutorName, *Message.ToString()
		);

		// Set the processing state to anything other than Pending, to prevent other executors picking up the same bad job.
		InJob->SetProcessingState(FIngestJob::FProcessingState{FIngestJob::EProcessingState::Aborted, MoveTemp(Message)});
		InPromise->SetValue();
		return nullptr;
	}

	FIngestProcessFinishReporter ProcessFinished = FIngestProcessFinishReporter::Type::CreateLambda(
		[This = AsShared(), Promise = InPromise, Job = InJob](const UIngestCapability_ProcessHandle* InProcessHandle, TValueOrError<void, FIngestCapability_Error> IngestError) mutable
		{
			FIngestJob::EProcessingState ProcessingState;
			FText Message;

			bool bIsDone = InProcessHandle->IsDone();

			if (IngestError.HasError())
			{
				ProcessingState = FIngestJob::EProcessingState::Aborted;
				Message = FText::Format(LOCTEXT("IngestProcessStateMessage_Aborted", "Ingest failed : {0} (Error code : {1})"), FText::FromString(IngestError.GetError().GetMessage()), FText::AsNumber(IngestError.GetError().GetCode()));
				UE_LOGF(LogCaptureManagerIngestJobExecutor, Error, "[%ls] %ls", *This->ExecutorName, *Message.ToString());
				bIsDone = true;
			}
			else if (bIsDone)
			{
				ProcessingState = FIngestJob::EProcessingState::Complete;
				Job->SetProgress(1.0);
				Message = LOCTEXT("IngestProcessStateMessage_Complete", "Success");
			}
			else
			{
				ProcessingState = FIngestJob::EProcessingState::Running;
				Message = LOCTEXT("IngestProcessStateMessage_Running", "Running...");
			}

			Job->SetProcessingState(FIngestJob::FProcessingState{ ProcessingState, MoveTemp(Message) });
			This->JobProcessingStateChanged.ExecuteIfBound(Job->GetGuid(), ProcessingState);
			
			if (bIsDone)
			{
				Promise->SetValue();
			}
		}
	);

	FIngestProcessProgressReporter ProgressReporter = FIngestProcessProgressReporter::Type::CreateLambda(
		[Job = InJob, LastProgress = 0.0](const UIngestCapability_ProcessHandle* InProcessHandle, const double Progress) mutable
		{
			// A little sanity check that we're receiving updates in the expected order.
			check(Progress >= LastProgress);
			LastProgress = Progress;

			Job->SetProgress(Progress);
		}
	);

	TStrongObjectPtr<UIngestCapability_ProcessHandle> ProcessHandle(
		ILiveLinkDeviceCapability_Ingest::Execute_CreateIngestProcess(Device, InJob->GetTakeId(), InJob->GetPipelineConfig()));

	ProcessHandle->OnProcessFinishReporter() = MoveTemp(ProcessFinished);
	ProcessHandle->OnProcessProgressReporter() = MoveTemp(ProgressReporter);

	ILiveLinkDeviceCapability_Ingest::Execute_RunIngestProcess(Device, ProcessHandle.Get(), IngestOptions);

	TUniquePtr<FRunIngestContext> Context = MakeUnique<FRunIngestContext>();
	Context->Device = TStrongObjectPtr<ULiveLinkDevice>(Device);
	Context->ProcessHandle = MoveTemp(ProcessHandle);
	return Context;
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE
