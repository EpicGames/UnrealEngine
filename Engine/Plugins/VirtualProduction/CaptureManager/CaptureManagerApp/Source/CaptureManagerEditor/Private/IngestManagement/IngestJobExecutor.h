// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IngestJob.h"

class UIngestCapability_ProcessHandle;

namespace UE::CaptureManager
{
class FIngestJobQueue;

class FIngestJobExecutor : public FRunnable, public TSharedFromThis<FIngestJobExecutor>
{
private:
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	DECLARE_DELEGATE(FOnComplete);
	DECLARE_DELEGATE_TwoParams(FJobProcessingStateChanged, FGuid, FIngestJob::EProcessingState);

	static TSharedRef<FIngestJobExecutor> Create(
		FString InExecutorName,
		TSharedRef<FIngestJobQueue> InProcessingQueue,
		FOnComplete InOnComplete,
		FJobProcessingStateChanged InJobProcessingStateChanged
	);

	FIngestJobExecutor(
		FPrivateToken InPrivateToken,
		FString InExecutorName,
		TSharedRef<FIngestJobQueue> InProcessingQueue,
		FOnComplete InOnComplete,
		FJobProcessingStateChanged InJobProcessingStateChanged
	);

	~FIngestJobExecutor();

	FIngestJobExecutor(const FIngestJobExecutor&) = delete;
	FIngestJobExecutor& operator=(const FIngestJobExecutor&) = delete;

	FIngestJobExecutor(FIngestJobExecutor&&) = delete;
	FIngestJobExecutor operator=(FIngestJobExecutor&&) = delete;

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	void CancelForDevice(const FGuid& InDeviceId);

private:
	struct FRunIngestContext
	{
		TStrongObjectPtr<ULiveLinkDevice> Device;
		TStrongObjectPtr<UIngestCapability_ProcessHandle> ProcessHandle;
	};

	TUniquePtr<FIngestJobExecutor::FRunIngestContext> RunIngest(TSharedRef<FIngestJob> InJob, TSharedRef<TPromise<void>> InPromise);

	FCriticalSection CriticalSection;
	FString ExecutorName;
	TSharedRef<FIngestJobQueue> ProcessingQueue;
	TUniquePtr<FIngestJobExecutor::FRunIngestContext> RunIngestProcessContext;
	FOnComplete Complete;
	FJobProcessingStateChanged JobProcessingStateChanged;

	std::atomic<bool> bIsRunning;
	std::atomic<bool> bStopRequested;
	TUniquePtr<FRunnableThread> Thread;
};

}
