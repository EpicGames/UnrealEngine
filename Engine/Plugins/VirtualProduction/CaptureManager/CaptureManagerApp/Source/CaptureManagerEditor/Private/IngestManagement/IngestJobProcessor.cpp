// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestJobProcessor.h"

#include "IngestJobExecutor.h"
#include "IngestJobQueue.h"

#include "CaptureManagerUnrealEndpointModule.h"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerIngestJobProcessor, Log, All);

namespace UE::CaptureManager
{

namespace Private
{

	static void StopUnrealEndpointConnections()
	{
		FCaptureManagerUnrealEndpointModule* EndpointModule = 
			FModuleManager::GetModulePtr<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint");
		if (!EndpointModule)
		{
			return;
		}
		
		TOptional<TSharedRef<FUnrealEndpointManager>> MaybeUnrealEndpointManager = EndpointModule->GetEndpointManagerIfValid();
		if (!MaybeUnrealEndpointManager.IsSet())
		{
			return;
		}

		TSharedRef<FUnrealEndpointManager> UnrealEndpointManager = MaybeUnrealEndpointManager.GetValue();
		const TArray<TWeakPtr<FUnrealEndpoint>> WeakEndpoints = UnrealEndpointManager->GetEndpoints();

		for (const TWeakPtr<FUnrealEndpoint>& WeakEndpoint : WeakEndpoints)
		{
			if (const TSharedPtr<FUnrealEndpoint> Endpoint = WeakEndpoint.Pin())
			{
				Endpoint->StopConnection();
			}
		}

		// Give any extant endpoints a chance to close their connections
		for (const TWeakPtr<FUnrealEndpoint>& WeakEndpoint : WeakEndpoints)
		{
			if (const TSharedPtr<FUnrealEndpoint> Endpoint = WeakEndpoint.Pin())
			{
				const bool bIsStopped = Endpoint->WaitForConnectionState(FUnrealEndpoint::EConnectionState::Disconnected, 3'000);

				if (!bIsStopped)
				{
					FString EndpointInfoString = UnrealEndpointInfoToString(Endpoint->GetInfo());
					UE_LOGF(LogCaptureManagerIngestJobProcessor, Warning, "Failed to stop unreal endpoint: %ls", *EndpointInfoString);
				}
			}
		}
	}
} // namespace Private

TSharedRef<FIngestJobProcessor> FIngestJobProcessor::MakeInstance(int32 NumExecutors)
{
	return MakeShared<FIngestJobProcessor>(FPrivateToken(), NumExecutors);
}

FIngestJobProcessor::FIngestJobProcessor(FPrivateToken PrivateToken, int32 InNumExecutors) :
	NumExecutors(InNumExecutors),
	ProcessingQueue(MakeShared<FIngestJobQueue>())
{
}

FIngestJobProcessor::~FIngestJobProcessor() = default;

void FIngestJobProcessor::StartProcessing()
{
	using namespace UE::CaptureManager::Private;

	if (IsProcessing())
	{
		return;
	}

	check(NumExecutorsRunning == 0);
	bStopRequested = false;

	{
		FScopeLock Lock(&CriticalSection);
		Executors.Empty();
	}

	ProcessingStateChanged.ExecuteIfBound(EProcessingState::Processing);

	for (int32 NumExecutor = 0; NumExecutor < NumExecutors; ++NumExecutor)
	{
		FIngestJobExecutor::FOnComplete OnComplete = FIngestJobExecutor::FOnComplete::CreateLambda(
			[this]()
			{
				if (--NumExecutorsRunning == 0)
				{
					// No more executors running, processing is complete.
					
					// Stop all unreal endpoints which have been started. Block until this is complete
					StopUnrealEndpointConnections();

					ProcessingStateChanged.ExecuteIfBound(EProcessingState::NotProcessing);
				}
			}
		);

		FIngestJobExecutor::FJobProcessingStateChanged OnJobProcessingStateChanged = FIngestJobExecutor::FJobProcessingStateChanged::CreateLambda(
			[this](const FGuid JobGuid, const FIngestJob::EProcessingState ProcessingState)
			{
				JobProcessingStateChanged.ExecuteIfBound(JobGuid, ProcessingState);
			}
		);

		++NumExecutorsRunning;

		{
			FScopeLock Lock(&CriticalSection);
			Executors.Emplace(
				FIngestJobExecutor::Create(
					FString::Printf(TEXT("Ingest Executor %d"), NumExecutor + 1),
					ProcessingQueue,
					MoveTemp(OnComplete),
					MoveTemp(OnJobProcessingStateChanged)
				)
			);
		}
	}
}

void FIngestJobProcessor::StopProcessing()
{
	if (!IsProcessing())
	{
		return;
	}

	bStopRequested = true;

	FScopeLock Lock(&CriticalSection);

	for (const TSharedRef<FIngestJobExecutor>& Executor : Executors)
	{
		Executor->Stop();
	}

	// IsProcessing will return true until all the executors have terminated
}

bool FIngestJobProcessor::IsProcessing() const
{
	return NumExecutorsRunning > 0;
}

bool FIngestJobProcessor::IsStopping() const
{
	return bStopRequested && IsProcessing();
}

int32 FIngestJobProcessor::AddJobs(TArray<TSharedRef<FIngestJob>> InIngestJobs)
{
	TArray<TSharedRef<FIngestJob>> AddedJobs = ProcessingQueue->AddJobs(MoveTemp(InIngestJobs));
	const int32 NumJobsAdded = AddedJobs.Num();
	JobsAdded.ExecuteIfBound(MoveTemp(AddedJobs));

	return NumJobsAdded;
}

bool FIngestJobProcessor::SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings InSettings)
{
	return ProcessingQueue->SetJobSettings(JobGuid, MoveTemp(InSettings));
}

uint32 FIngestJobProcessor::CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags)
{
	return ProcessingQueue->CountQueuedDeviceJobs(InDeviceId, InJobsToCountFlags);
}

int32 FIngestJobProcessor::RemoveJobsForDevice(const FGuid InDeviceId)
{
	TArray<FGuid> JobsGuidsRemoved = ProcessingQueue->RemoveJobsForDevice(InDeviceId);

	if (JobsGuidsRemoved.Num() > 0)
	{
		JobsRemoved.ExecuteIfBound(JobsGuidsRemoved);
	}

	return JobsGuidsRemoved.Num();
}

void FIngestJobProcessor::StopProcessingForDevice(const FGuid& InDeviceId)
{
	if (!IsProcessing())
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);

	for (const TSharedRef<FIngestJobExecutor>& Executor : Executors)
	{
		Executor->CancelForDevice(InDeviceId);
	}
}

int32 FIngestJobProcessor::RemoveJobs(const TArray<FGuid>& JobGuidsToRemove)
{
	if (JobGuidsToRemove.IsEmpty())
	{
		return 0;
	}

	TArray<FGuid> JobsGuidsRemoved = ProcessingQueue->Remove(JobGuidsToRemove);
	check(JobsGuidsRemoved == JobGuidsToRemove);

	if (JobsGuidsRemoved.Num() > 0)
	{
		JobsRemoved.ExecuteIfBound(JobsGuidsRemoved);
	}

	return JobsGuidsRemoved.Num();
}

int32 FIngestJobProcessor::RemoveAllJobs()
{
	if (IsProcessing())
	{
		return 0;
	}

	TArray<FGuid> JobsGuidsRemoved = ProcessingQueue->RemoveAll();
	JobsRemoved.ExecuteIfBound(MoveTemp(JobsGuidsRemoved));

	return JobsGuidsRemoved.Num();
}

FIngestJobProcessor::FJobsAdded& FIngestJobProcessor::OnJobsAdded()
{
	return JobsAdded;
}

FIngestJobProcessor::FJobsRemoved& FIngestJobProcessor::OnJobsRemoved()
{
	return JobsRemoved;
}

FIngestJobProcessor::FJobProcessingStateChanged& FIngestJobProcessor::OnJobProcessingStateChanged()
{
	return JobProcessingStateChanged;
}

FIngestJobProcessor::FProcessingStateChanged& FIngestJobProcessor::OnProcessingStateChanged()
{
	return ProcessingStateChanged;
}

}