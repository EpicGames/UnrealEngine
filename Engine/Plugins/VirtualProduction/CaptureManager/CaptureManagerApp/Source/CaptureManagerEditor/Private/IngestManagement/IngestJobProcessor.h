// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IngestManagement/IngestJob.h"

namespace UE::CaptureManager
{

class FIngestJobExecutor;
class FIngestJobQueue;

class FIngestJobProcessor: public TSharedFromThis<FIngestJobProcessor>
{
	struct FPrivateToken { explicit FPrivateToken() = default; };
public:
	enum class EProcessingState
	{
		Processing,
		NotProcessing
	};

	DECLARE_DELEGATE_OneParam(FJobsAdded, TArray<TSharedRef<FIngestJob>>);
	DECLARE_DELEGATE_OneParam(FJobsRemoved, TArray<FGuid>);
	DECLARE_DELEGATE_TwoParams(FJobProcessingStateChanged, FGuid, FIngestJob::EProcessingState);
	DECLARE_DELEGATE_OneParam(FProcessingStateChanged, EProcessingState);

	static TSharedRef<FIngestJobProcessor> MakeInstance(int32 NumExecutors);

	FIngestJobProcessor(FPrivateToken PrivateToken, int32 NumExecutors);
	~FIngestJobProcessor();

	int32 AddJobs(TArray<TSharedRef<FIngestJob>> IngestJobs);
	uint32 CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags);
	int32 RemoveJobsForDevice(const FGuid InDeviceId);
	int32 RemoveJobs(const TArray<FGuid>& JobGuidsToRemove);
	int32 RemoveAllJobs();
	bool SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings Settings);

	void StartProcessing();
	void StopProcessing();
	void StopProcessingForDevice(const FGuid& InDeviceId);

	bool IsProcessing() const;
	bool IsStopping() const;

	FJobsAdded& OnJobsAdded();
	FJobsRemoved& OnJobsRemoved();
	FJobProcessingStateChanged& OnJobProcessingStateChanged();
	FProcessingStateChanged& OnProcessingStateChanged();

private:
	bool JobCanBeAdded(const FIngestJob& IngestJob) const;
	mutable FCriticalSection CriticalSection;

	const int32 NumExecutors;
	std::atomic<int32> NumExecutorsRunning = 0;
	TArray<TSharedRef<FIngestJobExecutor>> Executors;
	TSharedRef<FIngestJobQueue> ProcessingQueue;
	std::atomic<bool> bStopRequested = false;

	FJobsAdded JobsAdded;
	FJobsRemoved JobsRemoved;
	FJobProcessingStateChanged JobProcessingStateChanged;
	FProcessingStateChanged ProcessingStateChanged;
};

}