// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestJobQueue.h"

#define LOCTEXT_NAMESPACE "IngestJobQueue"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureIngestJobQueue, Log, All);

namespace UE::CaptureManager
{

TArray<FGuid> FIngestJobQueue::Remove(const TArray<FGuid> GuidsToRemove)
{
	TArray<FGuid> GuidsRemoved;
	GuidsRemoved.Reserve(GuidsToRemove.Num());

	FScopeLock Lock(&CriticalSection);

	for (const FGuid& GuidToRemove : GuidsToRemove)
	{
		const int32 NumRemoved = Jobs.RemoveAll(
			[&GuidToRemove](const TSharedRef<FIngestJob>& Job)
			{
				return Job->GetGuid() == GuidToRemove;
			}
		);

		check(NumRemoved == 1);

		if (NumRemoved > 0)
		{
			GuidsRemoved.Emplace(GuidToRemove);
		}
	}

	return GuidsRemoved;
}

TArray<FGuid> FIngestJobQueue::RemoveAll()
{
	FScopeLock Lock(&CriticalSection);

	TArray<FGuid> GuidsRemoved;
	GuidsRemoved.Reserve(Jobs.Num());

	while (!Jobs.IsEmpty())
	{
		TSharedRef<FIngestJob> Job = Jobs.Pop();
		GuidsRemoved.Emplace(Job->GetGuid());
	}

	return GuidsRemoved;
}
uint32 FIngestJobQueue::CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags)
{
	for (const TSharedRef<FIngestJob>& Job : Jobs)
	{
		if (Job->GetCaptureDeviceId() == InDeviceId)
		{
			if (static_cast<uint32>(Job->GetProcessingState().State) & InJobsToCountFlags)
			{
				return true;
			}
		}
	}
	return false;
}

TArray<FGuid> FIngestJobQueue::RemoveJobsForDevice(const FGuid InDeviceId)
{
	TArray<FGuid> GuidsRemoved;

	const int32 NumRemoved = Jobs.RemoveAll(
		[&InDeviceId, &GuidsRemoved](const TSharedRef<FIngestJob>& Job)
		{
			bool IsDeviceJob = Job->GetCaptureDeviceId() == InDeviceId;
			if (IsDeviceJob)
			{
				GuidsRemoved.Add(Job->GetGuid());
			}
			
			return IsDeviceJob;
		}
	);
	return GuidsRemoved;
}

TSharedPtr<FIngestJob> FIngestJobQueue::ClaimFirstPending(const FIngestJob::EProcessingState ProcessingState)
{
	check(ProcessingState != FIngestJob::EProcessingState::Pending);

	if (ProcessingState == FIngestJob::EProcessingState::Pending)
	{
		return nullptr;
	}

	FScopeLock Lock(&CriticalSection);

	for (const TSharedRef<FIngestJob>& Job : Jobs)
	{
		if (Job->GetProcessingState().State == FIngestJob::EProcessingState::Pending)
		{
			Job->SetProcessingState(FIngestJob::FProcessingState{ ProcessingState, FText(LOCTEXT("IngestProcessStateMessage_Pending","Pending...")) });
			return Job;
		}
	}

	return nullptr;
}

TArray<TSharedRef<FIngestJob>> FIngestJobQueue::AddJobs(const TArray<TSharedRef<FIngestJob>> InIngestJobs)
{
	TArray<TSharedRef<FIngestJob>> JobsAdded;
	JobsAdded.Reserve(InIngestJobs.Num());

	{
		FScopeLock Lock(&CriticalSection);

		for (const TSharedRef<FIngestJob>& IngestJob : InIngestJobs)
		{
			if (JobCanBeAdded(*IngestJob))
			{
				Jobs.Emplace(IngestJob);
				JobsAdded.Emplace(IngestJob);
			}
		}
	}

	return JobsAdded;
}

bool FIngestJobQueue::SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings InSettings)
{
	bool bUpdated = false;
	FScopeLock Lock(&CriticalSection);

	TSharedRef<FIngestJob>* Found = Jobs.FindByPredicate(
		[&JobGuid](const TSharedRef<FIngestJob>& Job)
		{
			return Job->GetGuid() == JobGuid;
		}
	);

	if (Found)
	{
		(*Found)->SetSettings(MoveTemp(InSettings));
		bUpdated = true;
	}

	return bUpdated;
}

bool FIngestJobQueue::JobCanBeAdded(const FIngestJob& JobToCheck) const
{
	const bool bAlreadyExists = Jobs.ContainsByPredicate(
		[&JobToCheck](const TSharedRef<FIngestJob>& IngestJob)
		{
			// This is a limitation ot the ingest process, everything is keyed around the take ID
			return IngestJob->GetCaptureDeviceId() == JobToCheck.GetCaptureDeviceId() &&
				IngestJob->GetTakeId() == JobToCheck.GetTakeId();
		}
	);

	if (bAlreadyExists)
	{
		UE_LOGF(
			LogCaptureIngestJobQueue,
			Warning,
			"Job could not be added to the queue, it already exists: %ls #%d",
			*JobToCheck.GetTakeMetadata().Slate,
			JobToCheck.GetTakeMetadata().TakeNumber
		);
	}

	return !bAlreadyExists;
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE
