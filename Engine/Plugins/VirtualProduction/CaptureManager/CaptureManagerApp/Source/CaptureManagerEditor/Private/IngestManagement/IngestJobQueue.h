// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IngestJob.h"

namespace UE::CaptureManager
{

class FIngestJobQueue
{
public:
	TArray<TSharedRef<FIngestJob>> AddJobs(TArray<TSharedRef<FIngestJob>> InIngestJobs);

	/**
	* Find the first pending job in the queue and return it.
	* 
	* Since multiple executors may be calling this function at the same time, we must immediately mark the job with a
	* new (non-pending) state, to prevent the same job getting picked up by multiple executors.
	* 
	* @param ProcessingState The new state for the returned job (must not be Pending!).
	*/
	TSharedPtr<FIngestJob> ClaimFirstPending(FIngestJob::EProcessingState ProcessingState);

	TArray<FGuid> Remove(TArray<FGuid> GuidsToRemove);
	TArray<FGuid> RemoveAll();
	uint32 CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags);
	TArray<FGuid> RemoveJobsForDevice(const FGuid InDeviceId);

	bool SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings InSettings);

private:
	bool JobCanBeAdded(const FIngestJob& JobToCheck) const;

	FCriticalSection CriticalSection;
	TArray<TSharedRef<FIngestJob>> Jobs;
};

}
