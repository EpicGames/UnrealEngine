// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestJobSettingsManager.h"

namespace UE::CaptureManager
{

FIngestJobSettingsManager::FIngestJobSettingsManager()
{
}

void FIngestJobSettingsManager::ApplyJobSpecificSettings(const FGuid& InJobGuid, const FIngestJob::FSettings& InSettings)
{
	FScopeLock Lock(&CriticalSection);

	TStrongObjectPtr<UIngestJobSettings>& Object = Settings.Emplace_GetRef(NewObject<UIngestJobSettings>());
	Object->JobGuid = InJobGuid;
	Object->AudioFileNamePrefix = InSettings.AudioSettings.FileNamePrefix;
	Object->AudioFormat = InSettings.AudioSettings.Format;
	Object->UploadHostName = InSettings.UploadHostName;
	Object->WorkingDirectory = FDirectoryPath(InSettings.WorkingDirectory);
	Object->DownloadFolder = FDirectoryPath(InSettings.DownloadFolder);
	Object->ImageFileNamePrefix = InSettings.VideoSettings.FileNamePrefix;
	Object->ImageFormat = InSettings.VideoSettings.Format;
	Object->ImagePixelFormat = InSettings.VideoSettings.ImagePixelFormat;
	Object->ImageRotation = InSettings.VideoSettings.ImageRotation;
}

int32 FIngestJobSettingsManager::RemoveSettings(const TArray<FGuid>& JobGuids)
{
	FScopeLock Lock(&CriticalSection);

	const int32 NumRemoved = Settings.RemoveAll(
		[&JobGuids](const TStrongObjectPtr<UIngestJobSettings>& Entry)
		{
			return JobGuids.Contains(Entry->JobGuid);
		}
	);

	return NumRemoved;
}

TArray<TWeakObjectPtr<UIngestJobSettings>> FIngestJobSettingsManager::GetSettings(const TArray<FGuid>& InJobGuids) const
{
	FScopeLock Lock(&CriticalSection);

	TArray<TWeakObjectPtr<UIngestJobSettings>> JobSettings;
	JobSettings.Reserve(InJobGuids.Num());

	for (const TStrongObjectPtr<UIngestJobSettings>& Entry : Settings)
	{
		if (InJobGuids.Contains(Entry->JobGuid))
		{
			JobSettings.Emplace(Entry.Get());
		}
	}

	return JobSettings;
}

} // namespace UE::CaptureManager