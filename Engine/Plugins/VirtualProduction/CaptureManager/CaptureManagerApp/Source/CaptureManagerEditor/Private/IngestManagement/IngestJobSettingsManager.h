// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIngestJobSettings.h"

#include "IngestJob.h"

namespace UE::CaptureManager
{

class FIngestJobSettingsManager
{
public:
	FIngestJobSettingsManager();

	void ApplyJobSpecificSettings(const FGuid& InJobGuid, const FIngestJob::FSettings& InSettings);
	TArray<TWeakObjectPtr<UIngestJobSettings>> GetSettings(const TArray<FGuid>& InJobGuids) const;

	int32 RemoveSettings(const TArray<FGuid>& InJobGuids);

private:
	mutable FCriticalSection CriticalSection;
	TArray<TStrongObjectPtr<UIngestJobSettings>> Settings;
};
}