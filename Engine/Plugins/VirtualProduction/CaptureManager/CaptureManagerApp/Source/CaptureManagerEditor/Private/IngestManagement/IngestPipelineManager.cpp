// Copyright Epic Games, Inc. All Rights Reserved.

#include "IngestPipelineManager.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "IngestPipelineManager"

namespace UE::CaptureManager
{

FIngestPipelineManager::FIngestPipelineManager()
{
	UEnum* ProcessConfig = StaticEnum<EIngestCapability_ProcessConfig>();

	for (int32 EnumIndex = 0; EnumIndex < ProcessConfig->NumEnums() - 1; ++EnumIndex) // -1 to skip the implicit "MAX"
	{
		if (ProcessConfig->HasMetaData(TEXT("Hidden"), EnumIndex))
		{
			continue;
		}

		FText DisplayName = ProcessConfig->GetDisplayNameTextByIndex(EnumIndex);
		FText Tooltip = ProcessConfig->GetToolTipTextByIndex(EnumIndex);
		EIngestCapability_ProcessConfig PipelineConfig = 
			static_cast<EIngestCapability_ProcessConfig>(ProcessConfig->GetValueByIndex(EnumIndex));

		FPipelineDetails PipelineDetails =
		{
			.DisplayName = MoveTemp(DisplayName),
			.ToolTip = MoveTemp(Tooltip),
			.PipelineConfig = PipelineConfig
		};

		if (PipelineConfig == EIngestCapability_ProcessConfig::Ingest)
		{
			SelectedPipeline = PipelineDetails;
		}

		Pipelines.Emplace(MoveTemp(PipelineDetails));
	}
}

const TArray<FPipelineDetails>& FIngestPipelineManager::GetPipelines() const
{
	return Pipelines;
}

TOptional<FPipelineDetails> FIngestPipelineManager::SelectPipelineByDisplayName(const FText& DisplayName)
{
	TOptional<FPipelineDetails> MaybePipeline = GetPipelineByDisplayName(DisplayName);

	if (MaybePipeline.IsSet())
	{
		SelectedPipeline = MoveTemp(MaybePipeline.GetValue());
		return SelectedPipeline;
	}

	return TOptional<FPipelineDetails>();
}

TOptional<FPipelineDetails> FIngestPipelineManager::GetPipelineByDisplayName(const FText& DisplayName)
{
	const FPipelineDetails* Pipeline = Pipelines.FindByPredicate(
		[&DisplayName](const FPipelineDetails& InPipelineDetails)
		{
			return InPipelineDetails.DisplayName.CompareTo(DisplayName) == 0;
		}
	);

	if (Pipeline)
	{
		return *Pipeline;
	}

	return TOptional<FPipelineDetails>();
}

const FPipelineDetails& FIngestPipelineManager::GetSelectedPipeline() const
{
	return SelectedPipeline;
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE