// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"

#include "Ingest/IngestCapability_ProcessHandle.h"

namespace UE::CaptureManager
{

struct FPipelineDetails
{
	FText DisplayName;
	FText ToolTip;
	EIngestCapability_ProcessConfig PipelineConfig;
};

class FIngestPipelineManager
{
public:
	FIngestPipelineManager();

	const TArray<FPipelineDetails>& GetPipelines() const;
	TOptional<FPipelineDetails> SelectPipelineByDisplayName(const FText& DisplayName);
	TOptional<FPipelineDetails> GetPipelineByDisplayName(const FText& DisplayName);
	const FPipelineDetails& GetSelectedPipeline() const;

private:
	FPipelineDetails SelectedPipeline;
	TArray<FPipelineDetails> Pipelines;
};

}