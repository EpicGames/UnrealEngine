// Copyright Epic Games, Inc. All Rights Reserved.

#include "UIngestJobSettings.h"

#include "Misc/Paths.h"

#include "Settings/CaptureManagerSettings.h"

#define LOCTEXT_NAMESPACE "UIngestJobSettings"

UIngestJobSettings::UIngestJobSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DefaultText", "Default");
	DownloadFolder = GetDefault<UCaptureManagerSettings>()->DownloadDirectory;
	WorkingDirectory = GetDefault<UCaptureManagerSettings>()->DefaultWorkingDirectory;
	ImageFormat = EOutputImageFormat::JPEG;
	ImageFileNamePrefix = TEXT("frame");
	ImagePixelFormat = EImagePixelFormat::U8_BGRA;
	ImageRotation = EImageRotation::None;
	AudioFormat = EAudioFormat::WAV;
	AudioFileNamePrefix = TEXT("audio");


	UploadHostName = GetDefault<UCaptureManagerSettings>()->DefaultUploadHostName;

	// Sanity check that the default upload host name was not empty. The capture manager settings initialization timing
	// can be tricky, so it's best to be sure.
	check(!UploadHostName.IsEmpty());
}

void UIngestJobSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (WorkingDirectory.Path.IsEmpty())
	{
		WorkingDirectory = GetDefault<UCaptureManagerSettings>()->DefaultWorkingDirectory;
	}

	if (DownloadFolder.Path.IsEmpty())
	{
		DownloadFolder = GetDefault<UCaptureManagerSettings>()->DownloadDirectory;
	}

	if (UploadHostName.IsEmpty())
	{
		UploadHostName = GetDefault<UCaptureManagerSettings>()->DefaultUploadHostName;
		check(!UploadHostName.IsEmpty());
	}
}

#undef LOCTEXT_NAMESPACE
