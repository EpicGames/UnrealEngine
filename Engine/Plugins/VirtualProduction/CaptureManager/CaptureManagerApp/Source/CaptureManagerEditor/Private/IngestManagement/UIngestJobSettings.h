// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UIngestJobSettings.generated.h"

// In order to enable a rather nice "reset to preset default" functionality in the job details view, we specify presets
// by inheriting from the base object and overriding the default property values in the constructor. If we simply 
// assign these values (post-construction) then these values are not treated as "default" and this functionality 
// doesn't work anymore.

UENUM()
enum class EOutputImageFormat : uint8
{
	JPEG = 0			UMETA(DisplayName = "jpg"),
	PNG					UMETA(DisplayName = "png"),
};

UENUM()
enum class EImagePixelFormat : uint8
{
	U8_BGRA = 0			UMETA(DisplayName = "U8 BGRA"),
	U8_Mono				UMETA(DisplayName = "U8 Mono")
};

UENUM()
enum class EImageRotation : uint8
{
	None = 0			UMETA(DisplayName = "None"),
	CW_90				UMETA(DisplayName = "Clockwise 90°"),
	CW_180				UMETA(DisplayName = "Clockwise 180°"),
	CW_270				UMETA(DisplayName = "Clockwise 270°"),
};

UENUM()
enum class EAudioFormat : uint8
{
	WAV = 0				UMETA(DisplayName = "wav")
};

UCLASS()
class UIngestJobSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(AdvancedDisplay)
	FGuid JobGuid;

	UPROPERTY(AdvancedDisplay)
	FText DisplayName;

	UPROPERTY(AdvancedDisplay)
	FDirectoryPath DownloadFolder;

	UPROPERTY(Category = Conversion, EditAnywhere)
	FDirectoryPath WorkingDirectory;

	UPROPERTY(Category = "Output|Video", EditAnywhere)
	EOutputImageFormat ImageFormat;

	UPROPERTY(Category = "Output|Video", EditAnywhere)
	FString ImageFileNamePrefix;

	UPROPERTY(Category = "Output|Video", EditAnywhere)
	EImagePixelFormat ImagePixelFormat;

	UPROPERTY(Category = "Output|Video", EditAnywhere)
	EImageRotation ImageRotation;

	UPROPERTY(Category = "Output|Audio", EditAnywhere)
	EAudioFormat AudioFormat;

	UPROPERTY(Category = "Output|Audio", EditAnywhere)
	FString AudioFileNamePrefix;

	UPROPERTY(Category = "Upload", EditAnywhere, Meta = (DisplayName = "Host Name", NoResetToDefault))
	FString UploadHostName;

	UIngestJobSettings(const FObjectInitializer& ObjectInitializer);

private:

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};
