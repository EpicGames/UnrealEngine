// Copyright Epic Games, Inc. All Rights Reserved.

#include "ULiveLinkHubCaptureDevice.h"

#include "LiveLinkDeviceCapability_Connection.h"
#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "Engine/Engine.h"

#include "LiveLinkDeviceSubsystem.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubScriptedIngest"

static const FText NoSessionText = LOCTEXT("NoSession", "No session");
static const FText StartTimedOutText = LOCTEXT("StartTimedOut", "Timed out waiting for device to start");
static const FText InvalidTakeSessionIdText = LOCTEXT("InvalidTakeSessionId", "Session ID for this take is incorrect, you can not refer to a take from previous device session");
static const FText IngestNotSupportedText = LOCTEXT("IngestNotSupported", "Device does not support ingest");

static const FText FailedToCreateCaptureSourceFormatText = LOCTEXT("FailedToCreateCaptureSourceFormat", "Failed to create capture source: {0}");
static const FText ImportFailedFormatText = LOCTEXT("ImportFailedFormat", "Import failed: {0}");

struct ULiveLinkHubCaptureDevice::FImpl
{
	struct FSession
	{
		explicit FSession(const TObjectPtr<ULiveLinkDevice> InDevice);

		// This session ID is used to verify that any take objects we have returned to the user still belong to the 
		// current start/stop session. This should prevent issues where take IDs have changed after a start/stop call 
		// on the device. As an example consider an archive capture source, it may register takes in a different order
		// depending on a non-deterministic traversal of the filesystem and so the same take from an earlier session
		// may now have a different take ID.
		const FGuid Id;

		FGuid CaptureDeviceId;
	};

	FImpl();

	TObjectPtr<UIngestCapability_ProcessResult> IngestTake(const FLiveLinkHubTakeMetadata& InTake, 
														   const TObjectPtr<const UIngestCapability_Options> InConversionSettings) const;
	TObjectPtr<UIngestCapability_ProcessResult> DownloadTake(const FLiveLinkHubTakeMetadata& InTake,
															 const TObjectPtr<const UIngestCapability_Options> InConversionSettings) const;

	TObjectPtr<UIngestCapability_ProcessResult> FetchTakes(TArray<FLiveLinkHubTakeMetadata>& OutTakes) const;

	TObjectPtr<UIngestCapability_ProcessResult> Start(const int32 InTimeoutSeconds);
	TObjectPtr<UIngestCapability_ProcessResult> Stop();

	bool StartCaptureDevice(int32 InTimeoutSeconds);
	void StopCaptureDevice();
	void RemoveDevice();

	TOptional<FSession> Session;

	TObjectPtr<ULiveLinkDevice> BackgroundDevice;

private:

	TObjectPtr<UIngestCapability_ProcessResult> RunProcessTake(const FLiveLinkHubTakeMetadata& InTake,
															   const TObjectPtr<const UIngestCapability_Options> InConversionSettings, 
															   EIngestCapability_ProcessConfig InProcessConfig) const;
};

ULiveLinkHubCaptureDevice::FImpl::FSession::FSession(const TObjectPtr<ULiveLinkDevice> InDevice) :
	Id(FGuid::NewGuid())
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	const FGuid* MaybeDevice = Subsystem->GetDeviceMap().FindKey(InDevice);
	check(MaybeDevice);

	CaptureDeviceId = *MaybeDevice;
}

ULiveLinkHubCaptureDevice::FImpl::FImpl()
{
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::Start(const int32 InTimeoutSeconds)
{
	using namespace UE::CaptureManager;

	if (Session)
	{
		return UIngestCapability_ProcessResult::Success();
	}

	const bool bIsStarted = StartCaptureDevice(InTimeoutSeconds);

	if (bIsStarted)
	{
		Session = FSession(BackgroundDevice);
		return UIngestCapability_ProcessResult::Success();
	}

	return UIngestCapability_ProcessResult::Error(StartTimedOutText);
}

bool ULiveLinkHubCaptureDevice::FImpl::StartCaptureDevice(const int32 InTimeoutSeconds)
{
	using namespace UE::CaptureManager;

	FEventRef ReachableEvent;

	UConnectionDelegate* ConnectionDelegate = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionDelegate(BackgroundDevice);
	FDelegateHandle DelegateHandle = 
		ConnectionDelegate->ConnectionChanged.AddLambda([&ReachableEvent](ELiveLinkDeviceConnectionStatus InEvent)
	{
		if (InEvent == ELiveLinkDeviceConnectionStatus::Connected)
		{
			ReachableEvent->Trigger();
		}
	});

	AsyncTask(
		ENamedThreads::AnyThread,
		[this]()
		{
			ILiveLinkDeviceCapability_Connection::Execute_Connect(BackgroundDevice);
		}
	);

	bool bResult = ReachableEvent->Wait(FTimespan::FromSeconds(InTimeoutSeconds));

	ConnectionDelegate->ConnectionChanged.Remove(DelegateHandle);

	return bResult;
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::Stop()
{
	using namespace UE::CaptureManager;

	if (!Session)
	{
		return UIngestCapability_ProcessResult::Success();
	}

	StopCaptureDevice();
	RemoveDevice();
	Session.Reset();

	return UIngestCapability_ProcessResult::Success();
}

void ULiveLinkHubCaptureDevice::FImpl::StopCaptureDevice()
{
	using namespace UE::CaptureManager;

	FEventRef UnreachableEvent;

	auto OnReachableEvent = [&UnreachableEvent](ELiveLinkDeviceConnectionStatus InEvent)
		{
			if (InEvent == ELiveLinkDeviceConnectionStatus::Disconnected)
			{
				UnreachableEvent->Trigger();
				return;
			}
		};

	UConnectionDelegate* ConnectionDelegate = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionDelegate(BackgroundDevice);
	FDelegateHandle DelegateHandle = ConnectionDelegate->ConnectionChanged.AddLambda(MoveTemp(OnReachableEvent));

	AsyncTask(
		ENamedThreads::AnyThread,
		[this]()
		{
			ILiveLinkDeviceCapability_Connection::Execute_Disconnect(BackgroundDevice);
		}
	);

	UnreachableEvent->Wait();

	ConnectionDelegate->ConnectionChanged.Remove(DelegateHandle);
}

void ULiveLinkHubCaptureDevice::FImpl::RemoveDevice()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	Subsystem->RemoveDevice(BackgroundDevice);
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::IngestTake(const FLiveLinkHubTakeMetadata& InTake, 
																						 TObjectPtr<const UIngestCapability_Options> InConversionSettings) const
{
	return RunProcessTake(InTake, MoveTemp(InConversionSettings), EIngestCapability_ProcessConfig::Ingest);
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::DownloadTake(const FLiveLinkHubTakeMetadata& InTake,
																						   TObjectPtr<const UIngestCapability_Options> InConversionSettings) const
{
	return RunProcessTake(InTake, MoveTemp(InConversionSettings), EIngestCapability_ProcessConfig::Download);
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::RunProcessTake(const FLiveLinkHubTakeMetadata& InTake,
																							 const TObjectPtr<const UIngestCapability_Options> InConversionSettings,
																							 EIngestCapability_ProcessConfig InProcessConfig) const
{
	using namespace UE::CaptureManager;

	if (!Session)
	{
		return UIngestCapability_ProcessResult::Error(NoSessionText);
	}

	if (InTake.SessionId != Session->Id)
	{
		return UIngestCapability_ProcessResult::Error(InvalidTakeSessionIdText);
	}

	if (!BackgroundDevice->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return UIngestCapability_ProcessResult::Error(IngestNotSupportedText);
	}

	FEventRef ImportCompleteEvent;
	bool bImportSuccess = false;
	FText ErrorMessage;

	TManagedDelegate<const UIngestCapability_ProcessHandle*, TValueOrError<void, FIngestCapability_Error>> OnIngestFinished(
		[&ImportCompleteEvent, &bImportSuccess, &ErrorMessage](const UIngestCapability_ProcessHandle* InProcessHandle, TValueOrError<void, FIngestCapability_Error> InImportResult)
		{
			bool bIsDone = InProcessHandle->IsDone();

			if (InImportResult.HasError())
			{
				ErrorMessage = FText::Format(ImportFailedFormatText, FText::FromString(InImportResult.GetError().GetMessage()));
				bIsDone = true;
			}
			else if (bIsDone)
			{
				bImportSuccess = true;
			}

			if (bIsDone)
			{
				ImportCompleteEvent->Trigger();
			}
		},
		EDelegateExecutionThread::InternalThread
	);

	TStrongObjectPtr<UIngestCapability_ProcessHandle> ProcessHandle(
		ILiveLinkDeviceCapability_Ingest::Execute_CreateIngestProcess(BackgroundDevice, InTake.TakeId, InProcessConfig));
	check(ProcessHandle);

	ProcessHandle->OnProcessFinishReporter() = MoveTemp(OnIngestFinished);

	ILiveLinkDeviceCapability_Ingest::Execute_RunIngestProcess(BackgroundDevice, ProcessHandle.Get(), InConversionSettings);

	ImportCompleteEvent->Wait();

	if (bImportSuccess)
	{
		return UIngestCapability_ProcessResult::Success();
	}

	return UIngestCapability_ProcessResult::Error(ErrorMessage);
}

TObjectPtr<UIngestCapability_ProcessResult> ULiveLinkHubCaptureDevice::FImpl::FetchTakes(TArray<FLiveLinkHubTakeMetadata>& OutTakes) const
{
	using namespace  UE::CaptureManager;

	if (!OutTakes.IsEmpty())
	{
		// Make sure the output array is in a valid state, in case anything goes wrong below
		OutTakes.Empty();
	}

	if (!Session)
	{
		return UIngestCapability_ProcessResult::Error(NoSessionText);
	}

	if (!BackgroundDevice->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return UIngestCapability_ProcessResult::Error(IngestNotSupportedText);
	}

	FEventRef TakeListUpdatedEvent;
	TArray<FLiveLinkHubTakeMetadata> Takes;

	TManagedDelegate<TArray<FTakeId>> OnTakesListUpdated(
		[SessionId = Session->Id, Device = BackgroundDevice, &Takes, &TakeListUpdatedEvent](const TArray<FTakeId>& InTakeIds)
		{
			for (const FTakeId& TakeId : InTakeIds)
			{
				UIngestCapability_TakeInformation* Metadata =  ILiveLinkDeviceCapability_Ingest::Execute_GetTakeInformation(Device, TakeId);

				FLiveLinkHubTakeMetadata Take;
				Take.TakeId = TakeId;
				Take.SessionId = SessionId;
				Take.Metadata = Metadata;

				Takes.Emplace(MoveTemp(Take));
			}

			TakeListUpdatedEvent->Trigger();
		},
		EDelegateExecutionThread::InternalThread
	);

	UIngestCapability_UpdateTakeListCallback* UpdateTakeListCallback = NewObject<UIngestCapability_UpdateTakeListCallback>();
	check(UpdateTakeListCallback);

	UpdateTakeListCallback->Callback = MoveTemp(OnTakesListUpdated);

	ILiveLinkDeviceCapability_Ingest::Execute_UpdateTakeList(BackgroundDevice, UpdateTakeListCallback);
	TakeListUpdatedEvent->Wait();

	OutTakes = MoveTemp(Takes);

	return UIngestCapability_ProcessResult::Success();
}

ULiveLinkHubCaptureDevice* ULiveLinkHubCaptureDeviceFactory::CreateDeviceByClass(FString InName, UClass* InDeviceClass, ULiveLinkDeviceSettings* InSettings)
{
	TSubclassOf<ULiveLinkDevice> DeviceClass = InDeviceClass;

	ULiveLinkDeviceSubsystem* DeviceSubsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	check(DeviceSubsystem);

	ULiveLinkDeviceSubsystem::FCreateResult DeviceResult = DeviceSubsystem->CreateDeviceOfClass(DeviceClass, InSettings);
	ULiveLinkDevice* NewDevice = DeviceResult.GetValue().Device;

	check(NewDevice->Implements<ULiveLinkDeviceCapability_Ingest>());

	TObjectPtr<ULiveLinkHubCaptureDevice> Device = NewObject<ULiveLinkHubCaptureDevice>(this);
	Device->Name = MoveTemp(InName);
	Device->ImplPtr->BackgroundDevice = MoveTemp(NewDevice);

	return Device;
}

ULiveLinkHubCaptureDevice::ULiveLinkHubCaptureDevice() :
	ImplPtr(MakePimpl<FImpl>())
{
}

ULiveLinkHubCaptureDevice::~ULiveLinkHubCaptureDevice() = default;

UIngestCapability_ProcessResult* ULiveLinkHubCaptureDevice::Start(const int32 InTimeoutSeconds)
{
	return ImplPtr->Start(InTimeoutSeconds);
}

UIngestCapability_ProcessResult* ULiveLinkHubCaptureDevice::Stop()
{
	return ImplPtr->Stop();
}

UIngestCapability_ProcessResult* ULiveLinkHubCaptureDevice::IngestTake(const FLiveLinkHubTakeMetadata& InTake, const UIngestCapability_Options* InConversionSettings) const
{
	TObjectPtr<const UIngestCapability_Options> ConversionSettings = InConversionSettings;
	return ImplPtr->IngestTake(InTake, MoveTemp(ConversionSettings));
}

UIngestCapability_ProcessResult* ULiveLinkHubCaptureDevice::DownloadTake(const FLiveLinkHubTakeMetadata& InTake, const FString& InDownloadDirectory) const
{
	TObjectPtr<UIngestCapability_Options> ConversionSettings = NewObject<UIngestCapability_Options>();
	ConversionSettings->DownloadDirectory = InDownloadDirectory;
	return ImplPtr->DownloadTake(InTake, MoveTemp(ConversionSettings));
}

FLiveLinkHubFetchTakesResult ULiveLinkHubCaptureDevice::FetchTakes() const
{
	TArray<FLiveLinkHubTakeMetadata> Takes;
	TObjectPtr<UIngestCapability_ProcessResult> Status = ImplPtr->FetchTakes(Takes);

	FLiveLinkHubFetchTakesResult Result;
	Result.Status = MoveTemp(Status);
	Result.Takes = MoveTemp(Takes);

	return Result;
}

#undef LOCTEXT_NAMESPACE
