// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Ingest/IngestCapability_TakeInformation.h"
#include "Ingest/IngestCapability_ProcessHandle.h"
#include "Ingest/IngestCapability_Options.h"

#include "LiveLinkDevice.h"

#include "ULiveLinkHubCaptureDevice.generated.h"

USTRUCT(BlueprintType)
struct FLiveLinkHubTakeMetadata
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Metadata)
	TObjectPtr<UIngestCapability_TakeInformation> Metadata;

	UPROPERTY()
	int32 TakeId = 0;

	UPROPERTY()
	/** Identifies which start/stop session this information belongs to, to ensure the TakeId is still valid */
	FGuid SessionId;
};

USTRUCT(BlueprintType)
struct FLiveLinkHubFetchTakesResult
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Result)
	TObjectPtr<UIngestCapability_ProcessResult> Status;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Result)
	TArray<FLiveLinkHubTakeMetadata> Takes;
};

UCLASS()
class ULiveLinkHubCaptureDeviceFactory : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Ingest | Creator")
	ULiveLinkHubCaptureDevice* CreateDeviceByClass(FString InName, UClass* InDeviceClass, ULiveLinkDeviceSettings* Settings);
};

UCLASS()
class ULiveLinkHubCaptureDevice : public UObject
{
	GENERATED_BODY()

public:
	ULiveLinkHubCaptureDevice();
	~ULiveLinkHubCaptureDevice();

	UFUNCTION(BlueprintCallable, Category = Ingest)
	UIngestCapability_ProcessResult* Start(int32 InTimeoutSeconds);

	UFUNCTION(BlueprintCallable, Category = Ingest)
	UIngestCapability_ProcessResult* Stop();

	UFUNCTION(BlueprintCallable, Category = Ingest)
	UIngestCapability_ProcessResult* IngestTake(const FLiveLinkHubTakeMetadata& InTake, const UIngestCapability_Options* InConversionSettings) const;

	UFUNCTION(BlueprintCallable, Category = Ingest)
	UIngestCapability_ProcessResult* DownloadTake(const FLiveLinkHubTakeMetadata& InTake, const FString& InDownloadDirectory) const;

	UFUNCTION(BlueprintCallable, Category = Ingest)
	FLiveLinkHubFetchTakesResult FetchTakes() const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Info)
	FString Name;

private:

	struct FImpl;
	TPimplPtr<FImpl> ImplPtr;

	friend ULiveLinkHubCaptureDeviceFactory;
};

