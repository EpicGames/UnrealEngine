// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerPanelController.h"

#include "CaptureManagerUnrealEndpointModule.h"
#include "TakesView.h"
#include "SIngestJobProcessor.h"

#include "IngestManagement/IngestJobSettingsManager.h"

#include "IDetailsView.h"
#include "PropertyEditorModule.h"

#include "Settings/CaptureManagerSettings.h"

#include "Engine/Engine.h"
#include "ILiveLinkDeviceModule.h"
#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"

#define LOCTEXT_NAMESPACE "CaptureManagerPanelController"

DEFINE_LOG_CATEGORY_STATIC(LogCaptureManagerPanelController, Log, All);

namespace UE::CaptureManager::Private
{
static int32 GetNumberOfIngestExecutors()
{
	const UCaptureManagerSettings* Settings = GetDefault<UCaptureManagerSettings>();
	check(Settings);

	// Fallback for safety, just in case we can't find the settings
	int32 NumExecutors = 1;

	if (Settings)
	{
		NumExecutors = Settings->NumIngestExecutors;
	}

	return NumExecutors;
}
}

TSharedPtr<FCaptureManagerPanelController> FCaptureManagerPanelController::MakeInstance()
{
	// We do this to enforce the use of a TSharedPtr and to hide CreateViews from the user
	TSharedPtr<FCaptureManagerPanelController> PanelController = MakeShared<FCaptureManagerPanelController>(FPrivateToken{});
	PanelController->CreateViews();
	return PanelController;
}

FCaptureManagerPanelController::FCaptureManagerPanelController(FPrivateToken InPrivateToken) :
	IngestPipelineManager(MakeShared<UE::CaptureManager::FIngestPipelineManager>()),
	UnrealEndpointManager(FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager()),
	IngestJobSettingsManager(MakeShared<UE::CaptureManager::FIngestJobSettingsManager>())
{
	using namespace UE::CaptureManager;

	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	Subsystem->OnDeviceAdded().AddRaw(this, &FCaptureManagerPanelController::OnDeviceAdded);
	Subsystem->OnDeviceRemoved().AddRaw(this, &FCaptureManagerPanelController::OnDeviceRemoved);

	UpdateTakeListManager = FUpdateTakeListManager::Create();

	UnrealEndpointManager->Start();
}

FCaptureManagerPanelController::~FCaptureManagerPanelController()
{
	if (FModuleManager::Get().IsModuleLoaded("LiveLinkDevice"))
	{
		ILiveLinkDeviceModule::Get().OnSelectionChanged().Remove(DeviceSelectionChangedHandle);
	}
	IngestJobDetailsView->OnFinishedChangingProperties().Remove(FinishedChangingPropertiesHandle);

	IngestJobProcessorWidget->OnJobsAdded().Unbind();
	IngestJobProcessorWidget->OnJobsRemoved().Unbind();
	IngestJobProcessorWidget->OnProcessingStateChanged().Unbind();
	IngestJobProcessorWidget->OnSelectionChanged().Unbind();

	UpdateTakeListManager = nullptr;

	UnrealEndpointManager->Stop();
}

void FCaptureManagerPanelController::CreateViews()
{
	CreateIngestManagementViews();

	TakesView = MakeShared<FTakesView>(AsShared());

	DeviceSelectionChangedHandle = ILiveLinkDeviceModule::Get().OnSelectionChanged().AddSP(this, &FCaptureManagerPanelController::OnDeviceSelectionChanged);
}

void FCaptureManagerPanelController::CreateIngestManagementViews()
{
	using namespace UE::CaptureManager;
	using namespace UE::CaptureManager::Private;

	const int32 NumExecutors = GetNumberOfIngestExecutors();
	IngestJobProcessorWidget = SNew(SIngestJobProcessor).NumExecutors(NumExecutors);
	IngestJobDetailsView = CreateIngestJobDetailsView().ToSharedPtr();
	FinishedChangingPropertiesHandle = IngestJobDetailsView->OnFinishedChangingProperties().AddRaw(this, &FCaptureManagerPanelController::OnFinishedEditingJobProperties);

	IngestJobProcessorWidget->OnJobsAdded().BindRaw(this, &FCaptureManagerPanelController::OnJobsAdded);
	IngestJobProcessorWidget->OnJobsRemoved().BindRaw(this, &FCaptureManagerPanelController::OnJobsRemoved);
	IngestJobProcessorWidget->OnProcessingStateChanged().BindRaw(this, &FCaptureManagerPanelController::OnProcessingStateChanged);
	IngestJobProcessorWidget->OnSelectionChanged().BindRaw(this, &FCaptureManagerPanelController::OnIngestJobSelectionChanged);
}

TSharedRef<UE::CaptureManager::SIngestJobProcessor> FCaptureManagerPanelController::GetIngestJobProcessorWidget() const
{
	return IngestJobProcessorWidget.ToSharedRef();
}

TSharedRef<SWidget> FCaptureManagerPanelController::GetIngestJobDetailsWidget() const
{
	return IngestJobDetailsView.ToSharedRef();
}

TSharedRef<UE::CaptureManager::FIngestPipelineManager> FCaptureManagerPanelController::GetIngestPipelineManager() const
{
	return IngestPipelineManager;
}

TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> FCaptureManagerPanelController::GetIngestJobSettingsManager() const
{
	return IngestJobSettingsManager;
}

TSharedRef<IDetailsView> FCaptureManagerPanelController::CreateIngestJobDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;

	TSharedRef<IDetailsView> MainPropertyView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	MainPropertyView->SetObject(nullptr);

	MainPropertyView->SetIsPropertyReadOnlyDelegate(
		FIsPropertyReadOnly().CreateLambda(
			[](const FPropertyAndParent&)
			{
				return true;
			}
		)
	);

	return MainPropertyView;
}

TSharedPtr<STakesView> FCaptureManagerPanelController::GetTakesView() const
{
	return TakesView->TakesTileView;
}

TObjectPtr<ULiveLinkDevice> FCaptureManagerPanelController::GetCaptureDevice(FGuid InDeviceId) const
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	if (IsValid(Subsystem))
	{
		const TObjectPtr<ULiveLinkDevice>* DevicePtr = Subsystem->GetDeviceMap().Find(InDeviceId);

		if (DevicePtr)
		{
			TObjectPtr<ULiveLinkDevice> Device = *DevicePtr;
			check(Device->Implements<ULiveLinkDeviceCapability_Ingest>());
			return Device;
		}
	}

	return nullptr;
}

TArray<TObjectPtr<ULiveLinkDevice>> FCaptureManagerPanelController::GetCaptureDevices() const
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	TArray<TObjectPtr<ULiveLinkDevice>> CaptureDevices;
	if (IsValid(Subsystem))
	{
		for (const TPair<FGuid, TObjectPtr<ULiveLinkDevice>>& DevicePair : Subsystem->GetDeviceMap())
		{
			TObjectPtr<ULiveLinkDevice> Device = DevicePair.Value;

			if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
			{
				continue;
			}

			CaptureDevices.Add(MoveTemp(Device));
		}
	}

	return CaptureDevices;
}

TSharedRef<FUpdateTakeListManager> FCaptureManagerPanelController::GetUpdateTakeListManager() const
{
	return UpdateTakeListManager.ToSharedRef();
}

void FCaptureManagerPanelController::OnDeviceSelectionChanged(ULiveLinkDevice* InDevice)
{
	TakesView->DeviceSelectionChanged(InDevice);
}

void FCaptureManagerPanelController::OnDeviceAdded(FGuid InDeviceId, ULiveLinkDevice* InDevice)
{
	if (!InDevice->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return;
	}

	if (InDevice->Implements<ULiveLinkDeviceCapability_Connection>())
	{
		UConnectionDelegate* Delegate = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionDelegate(InDevice);
		Delegate->ConnectionChanged.AddSP(AsShared(), &FCaptureManagerPanelController::OnReachableEvent, InDeviceId);
	}

	TakesView->CaptureDeviceAdded(InDevice);

	if (SourcesReachableMap.Contains(InDeviceId) && SourcesReachableMap[InDeviceId])
	{
		TakesView->CaptureDeviceStarted(InDeviceId);
	}
	else if (!SourcesReachableMap.Contains(InDeviceId))
	{
		ELiveLinkDeviceConnectionStatus ConnectionStatus = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionStatus(InDevice);

		bool bConnectionStatus = ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected;
		SourcesReachableMap.Add(InDeviceId, bConnectionStatus);

		if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
		{
			TakesView->CaptureDeviceStarted(InDeviceId);
		}
		else
		{
			ILiveLinkDeviceCapability_Connection::Execute_Connect(InDevice);
		}
	}
}

void FCaptureManagerPanelController::OnDeviceRemoved(FGuid InDeviceId, ULiveLinkDevice* InDevice)
{
	if (!InDevice->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return;
	}

	if (InDevice->Implements<ULiveLinkDeviceCapability_Connection>())
	{
		UConnectionDelegate* Delegate = ILiveLinkDeviceCapability_Connection::Execute_GetConnectionDelegate(InDevice);
		Delegate->ConnectionChanged.RemoveAll(this);
	}

	TakesView->CaptureDeviceRemoved(InDevice);
	SourcesReachableMap.Remove(InDeviceId);
}

void FCaptureManagerPanelController::OnReachableEvent(ELiveLinkDeviceConnectionStatus InStatus, FGuid InDeviceId)
{
	using namespace UE::CaptureManager;

	bool ConnectionStatus = InStatus == ELiveLinkDeviceConnectionStatus::Connected;
	SourcesReachableMap.Add(InDeviceId, ConnectionStatus);

	if (ConnectionStatus)
	{
		TakesView->CaptureDeviceStarted(InDeviceId);
	}
	else
	{
		IngestJobProcessorWidget->RemoveJobsForDevice(InDeviceId);
		TakesView->CaptureDeviceStopped(InDeviceId);
	}
}

void FCaptureManagerPanelController::OnJobsAdded(TArray<TSharedRef<UE::CaptureManager::FIngestJob>> IngestJobs)
{
	using namespace UE::CaptureManager;

	for (const TSharedRef<FIngestJob>& IngestJob : IngestJobs)
	{
		IngestJobSettingsManager->ApplyJobSpecificSettings(IngestJob->GetGuid(), IngestJob->GetSettings());
	}
}

void FCaptureManagerPanelController::OnJobsRemoved(const TArray<FGuid> JobGuids)
{
	const int32 NumRemoved = IngestJobSettingsManager->RemoveSettings(JobGuids);

	if (NumRemoved != JobGuids.Num())
	{
		UE_LOGF(
			LogCaptureManagerPanelController, Warning,
			"Some ingest job settings were not removed from the settings manager (%d out of %d removed)",
			NumRemoved,
			JobGuids.Num()
		);
	}
}

void FCaptureManagerPanelController::OnProcessingStateChanged(const UE::CaptureManager::FIngestJobProcessor::EProcessingState ProcessingState)
{
	using namespace UE::CaptureManager;

	check(IngestJobDetailsView);
	check(TakesView);
	check(TakesView->TakesTileView);

	const bool bIsProcessing = ProcessingState == FIngestJobProcessor::EProcessingState::Processing;

	AsyncTask(
		ENamedThreads::GameThread,
		[bIsProcessing, Controller = SharedThis(this)]()
		{
			Controller->IngestJobDetailsView->SetIsPropertyReadOnlyDelegate(
				FIsPropertyReadOnly().CreateLambda(
					[bIsProcessing]
					(const FPropertyAndParent&)
					{
						return bIsProcessing;
					}
				)
			);

			// We disable the "Add to Queue" button, as we currently do not allow editing of job settings while processing is running. This means
			// it does not really make sense to add a job to the queue unless its preset settings are perfect (you don't want to edit them).
			Controller->TakesView->TakesTileView->SetAddToQueueButtonEnabled(!bIsProcessing);
		}
	);
}

void FCaptureManagerPanelController::OnFinishedEditingJobProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::CaptureManager;

	for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); Index++)
	{
		if (const UIngestJobSettings* Settings = Cast<const UIngestJobSettings>(PropertyChangedEvent.GetObjectBeingEdited(Index)))
		{
			// It shouldn't be possible to set an empty upload host name
			check(!Settings->UploadHostName.IsEmpty());

			// Convert the UObject into something the ingest job can truly own
			FIngestJob::FSettings IngestJobSettings =
			{
				.WorkingDirectory = Settings->WorkingDirectory.Path,
				.DownloadFolder = Settings->DownloadFolder.Path,
				.VideoSettings =
				{
					.Format = Settings->ImageFormat,
					.FileNamePrefix = Settings->ImageFileNamePrefix,
					.ImagePixelFormat = Settings->ImagePixelFormat,
					.ImageRotation = Settings->ImageRotation
				},
				.AudioSettings =
				{
					.Format = Settings->AudioFormat,
					.FileNamePrefix = Settings->AudioFileNamePrefix
				},
				.UploadHostName = Settings->UploadHostName
			};

			if (!IngestJobProcessorWidget->SetJobSettings(Settings->JobGuid, MoveTemp(IngestJobSettings)))
			{
				UE_LOGF(LogCaptureManagerPanelController, Warning, "Failed to update job settings. Job may not exist or may already be in progress");
			}
		}
	}
}

void FCaptureManagerPanelController::OnIngestJobSelectionChanged(const TArray<FGuid>& InJobGuids)
{
	using namespace UE::CaptureManager;

	check(IngestJobDetailsView);

	TArray<TWeakObjectPtr<UIngestJobSettings>> SettingsForJobs = IngestJobSettingsManager->GetSettings(InJobGuids);

	TArray<TWeakObjectPtr<UObject>> ObjectsForDetailsView;
	ObjectsForDetailsView.Reserve(SettingsForJobs.Num());

	for (TWeakObjectPtr<UIngestJobSettings> SettingsForJob : SettingsForJobs)
	{
		ObjectsForDetailsView.Emplace(SettingsForJob);
	}

	const bool bIsProcessing = IngestJobProcessorWidget->IsProcessing();

	AsyncTask(
		ENamedThreads::GameThread,
		[DetailsView = IngestJobDetailsView, Objects = MoveTemp(ObjectsForDetailsView), bIsProcessing]()
		{
			DetailsView->SetObjects(Objects);
			DetailsView->SetIsPropertyReadOnlyDelegate(
				FIsPropertyReadOnly().CreateLambda(
					[bIsProcessing](const FPropertyAndParent&)
					{
						return bIsProcessing;
					}
				)
			);
		}
	);
}

#undef LOCTEXT_NAMESPACE
