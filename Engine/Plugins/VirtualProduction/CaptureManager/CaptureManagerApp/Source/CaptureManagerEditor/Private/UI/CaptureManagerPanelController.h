// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "TakeVirtualAsset.h"

#include "IngestManagement/IngestJobProcessor.h"
#include "CaptureManagerUnrealEndpointManager.h"

#include "UpdateTakeListManager.h"

#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"

namespace UE::CaptureManager
{
class FCaptureSourceManager;
class FIngestPipelineManager;
class FIngestJobSettingsManager;
class SIngestJobProcessor;
class FCaptureEvent;
}

class FCaptureSourcesView;
class FTakesQueueView;
class FTakesView;
class SCaptureSourceListView;
class STakesView;
class STakesQueueView;
struct FCatpureSourceUIEntry;
class IDetailsView;

namespace ESelectInfo { enum Type : int; }

class FCaptureManagerPanelController : public TSharedFromThis<FCaptureManagerPanelController, ESPMode::ThreadSafe>
{
private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	static TSharedPtr<FCaptureManagerPanelController> MakeInstance();

	explicit FCaptureManagerPanelController(FPrivateToken InPrivateToken);
	~FCaptureManagerPanelController();

	TSharedPtr<STakesView> GetTakesView() const;

	TSharedRef<UE::CaptureManager::SIngestJobProcessor> GetIngestJobProcessorWidget() const;
	TSharedRef<SWidget> GetIngestJobDetailsWidget() const;
	TSharedRef<UE::CaptureManager::FIngestPipelineManager> GetIngestPipelineManager() const;
	TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> GetIngestJobSettingsManager() const;

	TObjectPtr<ULiveLinkDevice> GetCaptureDevice(FGuid InDeviceId) const;
	TArray<TObjectPtr<ULiveLinkDevice>> GetCaptureDevices() const;

	TSharedRef<FUpdateTakeListManager> GetUpdateTakeListManager() const;

private:

	void OnDeviceSelectionChanged(ULiveLinkDevice* InDevice);
	void OnDeviceAdded(FGuid InDeviceId, ULiveLinkDevice* InDevice);
	void OnDeviceRemoved(FGuid InDeviceId, ULiveLinkDevice* InDevice);
	void OnReachableEvent(ELiveLinkDeviceConnectionStatus InStatus, FGuid InDeviceId);

	void CreateViews();
	void CreateIngestManagementViews();
	TSharedRef<IDetailsView> CreateIngestJobDetailsView();

	void OnJobsAdded(TArray<TSharedRef<UE::CaptureManager::FIngestJob>> IngestJobs);
	void OnJobsRemoved(const TArray<FGuid> JobGuids);
	void OnProcessingStateChanged(UE::CaptureManager::FIngestJobProcessor::EProcessingState ProcessingState);
	void OnFinishedEditingJobProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	void OnIngestJobSelectionChanged(const TArray<FGuid>& InJobGuids);

	TSharedPtr<FTakesView> TakesView;
	TMap<FGuid, TStrongObjectPtr<class UConnectionStatusChangedObject>> ConnectionStatusChangedDelegates;

	TMap<FGuid, bool> SourcesReachableMap;

	TSharedRef<UE::CaptureManager::FIngestPipelineManager> IngestPipelineManager;
	TSharedRef<UE::CaptureManager::FUnrealEndpointManager> UnrealEndpointManager;
	TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> IngestJobSettingsManager;
	TSharedPtr<UE::CaptureManager::SIngestJobProcessor> IngestJobProcessorWidget;
	TSharedPtr<IDetailsView> IngestJobDetailsView;
	FDelegateHandle FinishedChangingPropertiesHandle;
	FDelegateHandle DeviceSelectionChangedHandle;

	TSharedPtr<FUpdateTakeListManager> UpdateTakeListManager;
};