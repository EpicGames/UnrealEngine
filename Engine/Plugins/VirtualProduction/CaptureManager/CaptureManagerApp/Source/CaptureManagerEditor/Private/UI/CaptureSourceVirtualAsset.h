// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#include "CaptureSourceVirtualAsset.generated.h"

UCLASS()
class UCaptureSourceVirtualAsset : public UObject
{
public:

	GENERATED_BODY()

	FGuid Id;
};