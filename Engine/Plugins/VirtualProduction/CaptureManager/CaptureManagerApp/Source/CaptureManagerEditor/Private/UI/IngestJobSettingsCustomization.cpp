// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/IngestJobSettingsCustomization.h"

#include "UI/SIngestSelectableUnrealEndpoint.h"
#include "IngestManagement/UIngestJobSettings.h"

#include "DetailWidgetRow.h"

FIngestJobSettingsCustomization::FIngestJobSettingsCustomization() = default;
FIngestJobSettingsCustomization::~FIngestJobSettingsCustomization() = default;

TSharedRef<IDetailCustomization> FIngestJobSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FIngestJobSettingsCustomization);
}

void FIngestJobSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UIngestJobSettings, UploadHostName));
	IDetailPropertyRow* PropertyRow = DetailLayout.EditDefaultProperty(PropertyHandle);
	check(PropertyRow);

	if (PropertyRow)
	{
		PropertyRow->CustomWidget()
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SIngestSelectableUnrealEndpoint)
					.PropertyHandle(PropertyHandle.ToSharedPtr())
			];
	}
}
