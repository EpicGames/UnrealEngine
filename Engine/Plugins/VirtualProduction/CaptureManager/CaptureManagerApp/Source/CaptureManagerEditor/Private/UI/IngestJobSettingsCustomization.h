// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"

class FIngestJobSettingsCustomization : public IDetailCustomization
{
public:
	~FIngestJobSettingsCustomization();

	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	FIngestJobSettingsCustomization();
};

