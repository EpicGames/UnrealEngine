// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubCaptureManagerMode.h"

#include "CaptureManagerUnrealEndpointModule.h"
#include "CaptureManagerPanelController.h"
#include "CaptureUtilsModule.h"
#include "LiveLinkHubApplicationBase.h"
#include "LiveLinkClientPanelViews.h"
#include "LiveLinkPanelController.h"
#include "Network/NetworkMisc.h"
#include "SIngestJobProcessor.h"
#include "TakesView.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubCaptureManagerMode"

namespace UE::CaptureManager
{
	const FName TakeBrowserTabId("TakeBrowserTabId");
	const FName JobsListTabId("JobsListTabId");
	const FName JobDetailsTabId("JobDetailsTabId");
	const FName StatusBarId("StatusBarId");
}

struct FTakeBrowserTabSummoner : public FWorkflowTabFactory
{
public:
	FTakeBrowserTabSummoner(TSharedPtr<FLiveLinkHubApplicationBase> InHostingApp, TSharedPtr<FCaptureManagerPanelController> InPanelController)
		: FWorkflowTabFactory(UE::CaptureManager::TakeBrowserTabId, MoveTemp(InHostingApp))
		, PanelController(MoveTemp(InPanelController))
	{
		TabLabel = LOCTEXT("TakeBrowserTabLabel", "Take Browser");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

		bIsSingleton = true;
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return PanelController->GetTakesView().ToSharedRef();
	}

private:
	TSharedPtr<FCaptureManagerPanelController> PanelController;
};

struct FJobsListTabSummoner : public FWorkflowTabFactory
{
public:
	FJobsListTabSummoner(TSharedPtr<FLiveLinkHubApplicationBase> InHostingApp, TSharedPtr<FCaptureManagerPanelController> InPanelController)
		: FWorkflowTabFactory(UE::CaptureManager::JobsListTabId, MoveTemp(InHostingApp))
		, PanelController(MoveTemp(InPanelController))
	{
		TabLabel = LOCTEXT("JobsListTabLabel", "Jobs List");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

		bIsSingleton = true;
	}

	virtual ~FJobsListTabSummoner() 
	{
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return PanelController->GetIngestJobProcessorWidget();
	}

private:
	TSharedPtr<FCaptureManagerPanelController> PanelController;
};

struct FJobsDetailsTabSummoner : public FWorkflowTabFactory
{
public:
	FJobsDetailsTabSummoner(TSharedPtr<FLiveLinkHubApplicationBase> InHostingApp, TSharedPtr<FCaptureManagerPanelController> InPanelController)
		: FWorkflowTabFactory(UE::CaptureManager::JobDetailsTabId, MoveTemp(InHostingApp))
		, CaptureManagerPanelController(MoveTemp(InPanelController))
	{
		TabLabel = LOCTEXT("JobDetailsTabLabel", "Job Details");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

		bIsSingleton = true;
	}

	virtual ~FJobsDetailsTabSummoner() {}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return CaptureManagerPanelController->GetIngestJobDetailsWidget();
	}

private:
	TSharedPtr<FCaptureManagerPanelController> CaptureManagerPanelController;
};


FLiveLinkHubCaptureManagerMode::FLiveLinkHubCaptureManagerMode(TSharedPtr<FLiveLinkHubApplicationBase> App)
	: FLiveLinkHubApplicationMode("CaptureManager", LOCTEXT("CaptureManagerModeLabel", "Capture Manager"), App),
	UnrealEndpointManager(FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager())
{
	using namespace UE::CaptureManager;

	TabLayout = FTabManager::NewLayout("LiveLinkCaptureManagerMode_v1.3");
	
	TabLayout->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(1.f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(UE::LiveLinkHub::SubjectsTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(UE::LiveLinkHub::SourceDetailsTabId, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(TakeBrowserTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(JobsListTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(JobDetailsTabId, ETabState::OpenedTab)
					)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->AddTab(UE::LiveLinkHub::PlaybackTabId, ETabState::ClosedTab)
			)
		);
	
	CaptureManagerPanelController = FCaptureManagerPanelController::MakeInstance();

	TabFactories.RegisterFactory(MakeShared<FTakeBrowserTabSummoner>(App, CaptureManagerPanelController));
	TabFactories.RegisterFactory(MakeShared<FJobsListTabSummoner>(App, CaptureManagerPanelController));
	TabFactories.RegisterFactory(MakeShared<FJobsDetailsTabSummoner>(App, CaptureManagerPanelController));
}

FLiveLinkHubCaptureManagerMode::~FLiveLinkHubCaptureManagerMode() = default;

TArray<TSharedRef<SWidget>> FLiveLinkHubCaptureManagerMode::GetStatusBarWidgets_Impl()
{
	return {
		SNew(STextBlock)
		.Text(this, &FLiveLinkHubCaptureManagerMode::GetDiscoveredClientsText)
	};
}

FText FLiveLinkHubCaptureManagerMode::GetDiscoveredClientsText() const
{
	const int32 NumUnrealEndpoints = UnrealEndpointManager->GetNumEndpoints();
	return FText::Format(LOCTEXT("DiscoveredClientsLabel", "Unreal Clients: {0}"), NumUnrealEndpoints);
}

#undef LOCTEXT_NAMESPACE