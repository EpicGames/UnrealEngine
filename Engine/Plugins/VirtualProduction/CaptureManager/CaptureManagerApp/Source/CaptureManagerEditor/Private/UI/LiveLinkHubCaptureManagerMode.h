// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerUnrealEndpointManager.h"

#include "LiveLinkHubApplicationMode.h"

#include "Framework/Docking/TabManager.h"
#include "Templates/SharedPointer.h"

#include "Async/CaptureTimerManager.h"

#include "IMessageContext.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#include "CaptureManagerStyle.h"


class FCommunicationManager;
class SCaptureManagerMainTabView;
class SDockTab;

class FLiveLinkHubCaptureManagerMode : public FLiveLinkHubApplicationMode
{
public:
	FLiveLinkHubCaptureManagerMode(TSharedPtr<FLiveLinkHubApplicationBase> App);
	virtual ~FLiveLinkHubCaptureManagerMode();

	//~ Begin FLiveLinkHubApplicationMode interface
	virtual FSlateIcon GetModeIcon() const override
	{
		return FSlateIcon(FCaptureManagerStyle::Get().GetStyleSetName(), TEXT("CaptureManagerIcon"));
	}

	virtual TArray<TSharedRef<SWidget>> GetStatusBarWidgets_Impl() override;
	//~ End FLiveLinkHubApplicationMode interface
private:
	FText GetDiscoveredClientsText() const;

private:
	TSharedRef<UE::CaptureManager::FUnrealEndpointManager> UnrealEndpointManager;
	TSharedPtr<class FCaptureManagerPanelController> CaptureManagerPanelController;
};

