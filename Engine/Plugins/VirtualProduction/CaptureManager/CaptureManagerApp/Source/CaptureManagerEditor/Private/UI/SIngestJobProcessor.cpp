// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIngestJobProcessor.h"

#include "IngestManagement/IngestJobProcessor.h"

#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "SWarningOrErrorBox.h"
#include "Settings/CaptureManagerSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Notifications/SProgressBar.h"

#define LOCTEXT_NAMESPACE "SIngestJobProcessor"

DEFINE_LOG_CATEGORY_STATIC(LogSIngestJobProcessor, Log, All)

namespace UE::CaptureManager
{

static const FName SlateHeaderId = TEXT("Slate");
static const FName TakeHeaderId = TEXT("Take");
static const FName ProgressHeaderId = TEXT("Progress");
static const FName StatusHeaderId = TEXT("Status");
static const FName ActionHeaderId = TEXT("Action");

static const FText StopText = LOCTEXT("StopText", "Stop");
static const FText StartText = LOCTEXT("StartText", "Start");
static const FText ClearText = LOCTEXT("ClearText", "Clear");
static const FText SlateText = LOCTEXT("SlateText", "Slate");
static const FText TakeText = LOCTEXT("TakeText", "Take");
static const FText ProgressText = LOCTEXT("ProgressText", "Progress");
static const FText StatusText = LOCTEXT("StatusText", "Status");

namespace Private
{
TArray<TSharedRef<FIngestJob>> GetFilteredJobs(const TArray<TSharedRef<FIngestJob>>& InIngestJobs, const TArray<FGuid>& InFilteredJobGuids)
{
	TArray<TSharedRef<FIngestJob>> FilteredJob;
	for (const TSharedRef<FIngestJob>& IngestJob : InIngestJobs)
	{
		if (InFilteredJobGuids.Contains(IngestJob->GetGuid()))
		{
			FilteredJob.Add(MakeShared<FIngestJob>(
				IngestJob->GetCaptureDeviceId(),
				IngestJob->GetTakeId(),
				IngestJob->GetTakeMetadata(),
				IngestJob->GetPipelineConfig(),
				IngestJob->GetSettings()
			));
		}
	}

	return FilteredJob;
}

void ResetJobs(TArray<TSharedRef<FIngestJob>> InIngestJobs, TSharedPtr<FIngestJobProcessor> InIngestJobProcessor)
{
	TArray<FGuid> AbortedJobs;
	Algo::TransformIf(InIngestJobs, AbortedJobs,
					  [](const TSharedRef<FIngestJob>& Input) -> bool
					  {
						  return Input->GetProcessingState().State == FIngestJob::EProcessingState::Aborted;
					  },
					  [](const TSharedRef<FIngestJob>& Input) -> FGuid
					  {
						  return Input->GetGuid();
					  });

	if (AbortedJobs.IsEmpty())
	{
		return;
	}

	if (!InIngestJobProcessor->RemoveJobs(AbortedJobs))
	{
		UE_LOGF(LogSIngestJobProcessor, Warning, "Failed to remove some jobs from the queue");
	}

	TArray<TSharedRef<FIngestJob>> FilteredJobs = GetFilteredJobs(InIngestJobs, AbortedJobs);

	check(!FilteredJobs.IsEmpty()); // FilteredJobs can't be empty if AbortedJobs isn't empty

	if (!InIngestJobProcessor->AddJobs(MoveTemp(FilteredJobs)))
	{
		UE_LOGF(LogSIngestJobProcessor, Warning, "Failed to add some jobs to the queue");
	}
}
}

class SIngestJobRow : public SMultiColumnTableRow<TSharedRef<FIngestJob>>
{
public:
	SLATE_BEGIN_ARGS(SIngestJobRow) {}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedRef<FIngestJob> IngestJob,
		TSharedRef<FIngestJobProcessor> IngestJobProcessor
	);

private:
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	TSharedRef<SWidget> CreateSlateNameWidget();
	TSharedRef<SWidget> CreateTakeNumberWidget();
	TSharedRef<SWidget> CreateProgressWidget();
	TSharedRef<SWidget> CreateStatusWidget();
	TSharedRef<SWidget> CreateActionWidget();

	FReply RetryButtonClicked();

	TSharedPtr<FIngestJob> IngestJob;
	TSharedPtr<FIngestJobProcessor> IngestJobProcessor;
};

void SIngestJobProcessor::Construct(const FArguments& Args)
{
	IngestJobProcessor = UE::CaptureManager::FIngestJobProcessor::MakeInstance(Args._NumExecutors);

	IngestJobProcessor->OnJobsAdded().BindRaw(this, &SIngestJobProcessor::HandleJobsAdded);
	IngestJobProcessor->OnJobsRemoved().BindRaw(this, &SIngestJobProcessor::HandleJobsRemoved);
	IngestJobProcessor->OnJobProcessingStateChanged().BindRaw(this, &SIngestJobProcessor::HandleJobProcessingStateChanged);
	IngestJobProcessor->OnProcessingStateChanged().BindRaw(this, &SIngestJobProcessor::HandleProcessingStateChanged);

	IngestJobsView = SNew(SListView<TSharedRef<FIngestJob>>)
		.ListItemsSource(&IngestJobs)
		.OnGenerateRow(this, &SIngestJobProcessor::OnGenerateWidgetForList)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged(this, &SIngestJobProcessor::HandleSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(SlateHeaderId)
			.VAlignCell(VAlign_Center)
			.HAlignCell(HAlign_Left)
			.FillWidth(0.4)
			[
				SNew(STextBlock)
					.Text(SlateText)
			]
			+ SHeaderRow::Column(TakeHeaderId)
			.FillWidth(0.1)
			.VAlignCell(VAlign_Center)
			.HAlignCell(HAlign_Right)
			[
				SNew(STextBlock)
					.Text(TakeText)
			]
			+ SHeaderRow::Column(ProgressHeaderId)
			.VAlignCell(VAlign_Center)
			.FillWidth(0.25)
			[
				SNew(STextBlock)
					.Text(ProgressText)
			]
			+ SHeaderRow::Column(StatusHeaderId)
			.FillWidth(0.15)
			.VAlignCell(VAlign_Center)
			.HAlignCell(HAlign_Center)
			[
				SNew(STextBlock)
					.Text(StatusText)
			]
			+ SHeaderRow::Column(ActionHeaderId)
			.FillWidth(0.1)
			.VAlignCell(VAlign_Center)
			.HAlignCell(HAlign_Center)
			[
				SNew(STextBlock)
			]
		);

	ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(2.0f)
				[
					IngestJobsView.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.Padding(2.0f)
				.AutoHeight()
				.VAlign(EVerticalAlignment::VAlign_Bottom)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						[
							SNew(SButton)
								.IsEnabled_Raw(this, &SIngestJobProcessor::ClearButtonIsEnabled)
								.Text(ClearText)
								.OnClicked_Raw(this, &SIngestJobProcessor::OnClearButtonClicked)
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.IsEnabled_Raw(this, &SIngestJobProcessor::StartButtonIsEnabled)
										.Text(StartText)
										.ToolTipText_Lambda([]
										{
#if !PLATFORM_WINDOWS
											if (!SIngestJobProcessor::IsThirdPartyEncoderConfigured())
											{
												return LOCTEXT("StartButton_DisabledTooltip",
													"Cannot start until Third Party Encoder is configured");
											}
#endif
											return LOCTEXT("StartButton_Tooltip", "Start ingest of the queued jobs.");
										})
										.OnClicked_Raw(this, &SIngestJobProcessor::OnStartButtonClicked)
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.VAlign(VAlign_Center)
										.IsEnabled_Raw(this, &SIngestJobProcessor::StopButtonIsEnabled)
										.Text(StopText)
										.OnClicked_Raw(this, &SIngestJobProcessor::OnStopButtonClicked)
								]
						]

				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 4.0f)
				[
					SNew(SWarningOrErrorBox)
						.Visibility(this, &SIngestJobProcessor::GetThirdPartyEncoderWarningVisibility)
						.MessageStyle(EMessageStyle::Warning)
						.Message(LOCTEXT("ThirdPartyEncoderNotConfigured", "Third Party Encoder is required to start processing"))
						[
							SNew(SButton)
								.TextStyle(FAppStyle::Get(), "DialogButtonText")
								.Text(LOCTEXT("ConfigureThirdPartyEncoderButton", "Configure"))
								.OnClicked_Lambda([]
								{
									FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
										.ShowViewer("Project", "Plugins", "Capture Manager");
									return FReply::Handled();
								})
						]
				]
		];
}

TSharedRef<ITableRow> SIngestJobProcessor::OnGenerateWidgetForList(TSharedRef<FIngestJob> InIngestJob, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SIngestJobRow, InOwnerTable, MoveTemp(InIngestJob), IngestJobProcessor.ToSharedRef());
}

SIngestJobProcessor::FSelectionChanged& SIngestJobProcessor::OnSelectionChanged()
{
	return SelectionChanged;
}

void SIngestJobProcessor::HandleJobsAdded(TArray<TSharedRef<FIngestJob>> InIngestJobs)
{
	{
		FScopeLock Lock(&CriticalSection);

		for (const TSharedRef<FIngestJob>& IngestJob : InIngestJobs)
		{
			IngestJobs.Emplace(IngestJob);
		}
	}

	AsyncTask(
		ENamedThreads::GameThread,
		[View = IngestJobsView]()
		{
			View->RebuildList();
		}
	);

	JobsAdded.ExecuteIfBound(InIngestJobs);
}

void SIngestJobProcessor::HandleJobsRemoved(const TArray<FGuid> RemovedGuids)
{
	{
		FScopeLock Lock(&CriticalSection);

		IngestJobs.RemoveAll(
			[&RemovedGuids](const TSharedRef<FIngestJob>& Job)
			{
				return RemovedGuids.Contains(Job->GetGuid());
			}
		);
	}

	AsyncTask(
		ENamedThreads::GameThread,
		[View = IngestJobsView]()
		{
			View->RebuildList();
		}
	);

	JobsRemoved.ExecuteIfBound(RemovedGuids);
}

void SIngestJobProcessor::HandleJobProcessingStateChanged(
	[[maybe_unused]] const FGuid JobGuid,
	[[maybe_unused]] FIngestJob::EProcessingState ProcessingState
)
{
	AsyncTask(
		ENamedThreads::GameThread,
		[View = IngestJobsView]()
		{
			View->RebuildList();
		}
	);

	JobProcessingStateChanged.ExecuteIfBound(JobGuid, ProcessingState);
}

void SIngestJobProcessor::HandleProcessingStateChanged(FIngestJobProcessor::EProcessingState ProcessingState)
{
	ProcessingStateChanged.ExecuteIfBound(ProcessingState);
}

void SIngestJobProcessor::HandleSelectionChanged(
	[[maybe_unused]] TSharedPtr<FIngestJob> InJobEntry,
	[[maybe_unused]] ESelectInfo::Type InSelectInfo
)
{
	TArray<TSharedRef<FIngestJob>> SelectedJobs = IngestJobsView->GetSelectedItems();

	TArray<FGuid> SelectedJobGuids;
	SelectedJobGuids.Reserve(SelectedJobs.Num());

	for (const TSharedRef<FIngestJob>& SelectedJob : SelectedJobs)
	{
		SelectedJobGuids.Emplace(SelectedJob->GetGuid());
	}

	SelectionChanged.ExecuteIfBound(SelectedJobGuids);
}

bool SIngestJobProcessor::ClearButtonIsEnabled() const
{
	return !IngestJobProcessor->IsProcessing();
}

FReply SIngestJobProcessor::OnClearButtonClicked()
{
	if (!IngestJobProcessor->IsProcessing())
	{
		IngestJobProcessor->RemoveAllJobs();
	}

	return FReply::Handled();
}

bool SIngestJobProcessor::StartButtonIsEnabled() const
{
	if (IngestJobProcessor->IsProcessing())
	{
		return false;
	}
#if !PLATFORM_WINDOWS
	if (!IsThirdPartyEncoderConfigured())
	{
		return false;
	}
#endif
	return true;
}

bool SIngestJobProcessor::IsThirdPartyEncoderConfigured()
{
	const UCaptureManagerSettings* Settings = GetDefault<UCaptureManagerSettings>();
	return Settings->bEnableThirdPartyEncoder && !Settings->ThirdPartyEncoder.FilePath.IsEmpty();
}

EVisibility SIngestJobProcessor::GetThirdPartyEncoderWarningVisibility() const
{
#if PLATFORM_WINDOWS
	return EVisibility::Collapsed;
#else
	return IsThirdPartyEncoderConfigured() ? EVisibility::Collapsed : EVisibility::Visible;
#endif
}

FReply SIngestJobProcessor::OnStartButtonClicked()
{
	FScopeLock Lock(&CriticalSection);

	Private::ResetJobs(IngestJobs, IngestJobProcessor);

	bool bShouldStartProcessing = !IngestJobs.IsEmpty();

	Lock.Unlock();

	if (bShouldStartProcessing)
	{
		IngestJobProcessor->StartProcessing();
	}

	return FReply::Handled();
}

bool SIngestJobProcessor::StopButtonIsEnabled() const
{
	return IngestJobProcessor->IsProcessing() && !IngestJobProcessor->IsStopping();
}

FReply SIngestJobProcessor::OnStopButtonClicked()
{
	IngestJobProcessor->StopProcessing();
	return FReply::Handled();
}

void SIngestJobRow::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& OwnerTable,
	TSharedRef<FIngestJob> InIngestJob,
	TSharedRef<FIngestJobProcessor> InIngestJobProcessor
)
{
	IngestJob = InIngestJob.ToSharedPtr();
	IngestJobProcessor = InIngestJobProcessor.ToSharedPtr();

	FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
	FSuperRowType::Construct(SuperArgs, OwnerTable);
}

TSharedRef<SWidget> SIngestJobRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SlateHeaderId)
	{
		return CreateSlateNameWidget();
	}

	if (ColumnName == TakeHeaderId)
	{
		return CreateTakeNumberWidget();
	}

	if (ColumnName == ProgressHeaderId)
	{
		return CreateProgressWidget();
	}

	if (ColumnName == StatusHeaderId)
	{
		return CreateStatusWidget();
	}

	if (ColumnName == ActionHeaderId)
	{
		return CreateActionWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SIngestJobRow::CreateSlateNameWidget()
{
	return SNew(SBox)
		.Padding(2.0f)
		[
			SNew(STextBlock).Text(FText::FromString(IngestJob->GetTakeMetadata().Slate))
		];
}

TSharedRef<SWidget> SIngestJobRow::CreateTakeNumberWidget()
{
	return SNew(SBox)
		.Padding(2.0f)
		[
			SNew(STextBlock)
				.Text(FText::FromString(FString::FromInt(IngestJob->GetTakeMetadata().TakeNumber)))
		];
}

TSharedRef<SWidget> SIngestJobRow::CreateProgressWidget()
{
	return SNew(SBox)
		.Padding(2.0f, 5.0f)
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SProgressBar)
						.BarFillType(EProgressBarFillType::LeftToRight)
						.Percent_Lambda(
							[this]()
							{
								return IngestJob->GetProgress();
							}
						)
				]
				// Downloading/ingesting caption
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
						.Margin(FMargin(0, 0))
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.TextStyle(FAppStyle::Get(), "ButtonText")
						.Justification(ETextJustify::Center)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.ColorAndOpacity(FSlateColor(FLinearColor::White))
						.Text_Lambda(
							[this]()
							{
								return FText::FromString(FString::Printf(TEXT("%.1f %%"), 100. * IngestJob->GetProgress()));
							}
						)
				]
		];
}

TSharedRef<SWidget> SIngestJobRow::CreateStatusWidget()
{
	TSharedPtr<SWidget> StateWidget;

	FIngestJob::EProcessingState State = IngestJob->GetProcessingState().State;
	switch (State)
	{
		case FIngestJob::EProcessingState::Running:
			StateWidget = SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
				.ColorAndOpacity(FStyleColors::Success)
				.ToolTipText(IngestJob->GetProcessingState().Message);
			break;

		case FIngestJob::EProcessingState::Complete:
			StateWidget = SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Success"))
				.ColorAndOpacity(FStyleColors::Success)
				.ToolTipText(IngestJob->GetProcessingState().Message);
			break;

		case FIngestJob::EProcessingState::Aborted:
			StateWidget = SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Alert"))
				.ColorAndOpacity(FStyleColors::Error)
				.ToolTipText(IngestJob->GetProcessingState().Message);
			break;

		default:
			StateWidget = SNullWidget::NullWidget;
			break;

	}

	return SNew(SScaleBox)
		.Stretch(EStretch::ScaleToFit)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			StateWidget.ToSharedRef()
		];
}

TSharedRef<SWidget> SIngestJobRow::CreateActionWidget()
{
	if (IngestJob->GetProcessingState().State == FIngestJob::EProcessingState::Aborted)
	{
		return SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
					.OnClicked(this, &SIngestJobRow::RetryButtonClicked)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Refresh"))
					]
			];
	}

	return SNullWidget::NullWidget;
}

FReply SIngestJobRow::RetryButtonClicked()
{
	Private::ResetJobs({ IngestJob.ToSharedRef()}, IngestJobProcessor);

	return FReply::Handled();
}

SIngestJobProcessor::FJobsAdded& SIngestJobProcessor::OnJobsAdded()
{
	return JobsAdded;
}

SIngestJobProcessor::FJobsRemoved& SIngestJobProcessor::OnJobsRemoved()
{
	return JobsRemoved;
}

SIngestJobProcessor::FJobProcessingStateChanged& SIngestJobProcessor::OnJobProcessingStateChanged()
{
	return JobProcessingStateChanged;
}

SIngestJobProcessor::FProcessingStateChanged& SIngestJobProcessor::OnProcessingStateChanged()
{
	return ProcessingStateChanged;
}

bool SIngestJobProcessor::IsProcessing() const
{
	return IngestJobProcessor->IsProcessing();
}

int32 SIngestJobProcessor::AddJobs(TArray<TSharedRef<FIngestJob>> Jobs)
{
	return IngestJobProcessor->AddJobs(MoveTemp(Jobs));
}

bool SIngestJobProcessor::SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings JobSettings)
{
	return IngestJobProcessor->SetJobSettings(JobGuid, MoveTemp(JobSettings));
}

uint32 SIngestJobProcessor::CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags)
{
	return IngestJobProcessor->CountQueuedDeviceJobs(InDeviceId, InJobsToCountFlags);
}

int32 SIngestJobProcessor::RemoveJobsForDevice(const FGuid InDeviceId)
{
	return IngestJobProcessor->RemoveJobsForDevice(InDeviceId);
}

void SIngestJobProcessor::Stop(const FGuid InDeviceId)
{
	IngestJobProcessor->StopProcessingForDevice(InDeviceId);
}

} // namespace UE::CaptureManager

#undef LOCTEXT_NAMESPACE
