// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IngestManagement/IngestJob.h"
#include "IngestManagement/IngestJobProcessor.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

namespace UE::CaptureManager
{

class SIngestJobProcessor : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FSelectionChanged, const TArray<FGuid>&);
	DECLARE_DELEGATE_OneParam(FJobsAdded, TArray<TSharedRef<FIngestJob>>);
	DECLARE_DELEGATE_OneParam(FJobsRemoved, TArray<FGuid>);
	DECLARE_DELEGATE_TwoParams(FJobProcessingStateChanged, FGuid, FIngestJob::EProcessingState);
	DECLARE_DELEGATE_OneParam(FProcessingStateChanged, FIngestJobProcessor::EProcessingState);

	SLATE_BEGIN_ARGS(SIngestJobProcessor) {}
		SLATE_ARGUMENT_DEFAULT(int32, NumExecutors) { 1 };
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	bool IsProcessing() const;
	int32 AddJobs(TArray<TSharedRef<FIngestJob>> Jobs);
	bool SetJobSettings(const FGuid& JobGuid, FIngestJob::FSettings JobSettings);
	uint32 CountQueuedDeviceJobs(const FGuid InDeviceId, const uint32 InJobsToCountFlags);
	int32 RemoveJobsForDevice(const FGuid InDeviceId);
	void Stop(const FGuid InDeviceId);

	FSelectionChanged& OnSelectionChanged();
	FJobsAdded& OnJobsAdded();
	FJobsRemoved& OnJobsRemoved();
	FJobProcessingStateChanged& OnJobProcessingStateChanged();
	FProcessingStateChanged& OnProcessingStateChanged();

private:
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedRef<FIngestJob> InJob, const TSharedRef<STableViewBase>& InOwnerTable);

	void HandleJobsAdded(TArray<TSharedRef<FIngestJob>> IngestJobs);
	void HandleJobsRemoved(TArray<FGuid> RemovedGuids);
	void HandleJobProcessingStateChanged(const FGuid JobGuid, FIngestJob::EProcessingState ProcessingState);
	void HandleProcessingStateChanged(FIngestJobProcessor::EProcessingState ProcessingState);
	void HandleSelectionChanged(TSharedPtr<FIngestJob> InJobEntry, ESelectInfo::Type InSelectInfo);

	bool ClearButtonIsEnabled() const;
	bool StartButtonIsEnabled() const;
	bool StopButtonIsEnabled() const;

	static bool IsThirdPartyEncoderConfigured();
	EVisibility GetThirdPartyEncoderWarningVisibility() const;

	FReply OnClearButtonClicked();
	FReply OnStartButtonClicked();
	FReply OnStopButtonClicked();

	FCriticalSection CriticalSection;
	TArray<TSharedRef<FIngestJob>> IngestJobs;
	TSharedPtr<SListView<TSharedRef<FIngestJob>>> IngestJobsView;
	TSharedPtr<FIngestJobProcessor> IngestJobProcessor;

	FSelectionChanged SelectionChanged;
	FJobsAdded JobsAdded;
	FJobsRemoved JobsRemoved;
	FJobProcessingStateChanged JobProcessingStateChanged;
	FProcessingStateChanged ProcessingStateChanged;
};

}