// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SIngestSelectableUnrealEndpoint.h"

#include "CaptureManagerUnrealEndpointModule.h"
#include "Network/NetworkMisc.h"

#include "IPropertyTypeCustomization.h"

#include "Async/Async.h"
#include "Modules/ModuleManager.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "IngestSelectableUnrealEndpoint"

namespace UE::CaptureManager::Private
{
struct FUnrealEndpointInfoComparator
{
	bool operator()(const FUnrealEndpointInfo& Lhs, const FUnrealEndpointInfo& Rhs) const
	{
		// Sort on hostname first
		if (Lhs.HostName < Rhs.HostName)
		{
			return true;
		}
		if (Rhs.HostName < Lhs.HostName)
		{
			return false;
		}

		// Then IP address
		if (Lhs.IPAddress < Rhs.IPAddress)
		{
			return true;
		}
		if (Rhs.IPAddress < Lhs.IPAddress)
		{
			return false;
		}

		// And finally on the import service port
		return Lhs.ImportServicePort < Rhs.ImportServicePort;
	}
};

static FText GetUnrealEndpointTooltip(TSharedRef<FUnrealEndpointInfo> EndpointInfo)
{
	FString ToolTip = FString::Printf(TEXT("IPAddress=%s, Port=%d"), *EndpointInfo->IPAddress, EndpointInfo->ImportServicePort);
	return FText::FromString(ToolTip);
}
}

SIngestSelectableUnrealEndpoint::SIngestSelectableUnrealEndpoint() :
	UnrealEndpointManager(FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager())
{
}

SIngestSelectableUnrealEndpoint::~SIngestSelectableUnrealEndpoint()
{
	UnrealEndpointManager->EndpointsChanged().Remove(EndpointsChangedDelegateHandle);
}

void SIngestSelectableUnrealEndpoint::Construct(const FArguments& InArgs)
{
	using namespace UE::CaptureManager;
	using namespace UE::CaptureManager::Private;

	PropertyHandle = InArgs._PropertyHandle;
	LocalHostName = GetLocalHostNameChecked();

	FSimpleDelegate PropertyChanged = FSimpleDelegate::CreateSP(this, &SIngestSelectableUnrealEndpoint::OnPropertyChanged);
	PropertyHandle->SetOnPropertyValueChanged(PropertyChanged);

	// Populate EndpointInfos before we hook into the "on change" delegate (so we can get future updates)
	TArray<TSharedRef<FUnrealEndpointInfo>> LatestEndpointInfos = GetLatestEndpointInfos();
	SetEndpointInfos(MoveTemp(LatestEndpointInfos));

	EndpointsChangedDelegateHandle = UnrealEndpointManager->EndpointsChanged().AddLambda(
		[this]()
		{
			// Keep the internal infos up to date. We do sorting etc. off the game thread.
			TArray<TSharedRef<FUnrealEndpointInfo>> LatestEndpointInfos = GetLatestEndpointInfos();

			// Make sure member variable updates happen on the game thread. The combobox refresh needs to happen there anyway 
			// and this avoids the need for a mutex
			AsyncTask(
				ENamedThreads::GameThread,
				[This = SharedThis(this), LatestEndpointInfos = MoveTemp(LatestEndpointInfos)]() mutable
				{
					This->SetEndpointInfos(MoveTemp(LatestEndpointInfos));
				}
			);
		}
	);

	// Manually trigger the property changed event to set the current host name from the property value. 
	PropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);

	ChildSlot
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.FillWidth(1.0)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SAssignNew(ComboBox, SComboBox<TSharedRef<FUnrealEndpointInfo>>)
						.InitiallySelectedItem(TargetEndpointInfo)
						.OptionsSource(&EndpointInfos)
						.OnGenerateWidget_Lambda(
							[this](TSharedRef<FUnrealEndpointInfo> InEndpointInfo)
							{
								return SNew(STextBlock)
									.Text_Lambda(
										[EndpointInfo = InEndpointInfo, LocalHostName = LocalHostName]()
										{
											if (EndpointInfo->HostName == LocalHostName)
											{
												// Highlight that this is a locally running endpoint
												return FText::FromString(FString::Printf(TEXT("%s (Local)"), *EndpointInfo->HostName));
											}

											return FText::FromString(EndpointInfo->HostName);
										}
									)
									.ToolTipText_Lambda(
										[EndpointInfo = InEndpointInfo]()
										{
											FText ToolTip = GetUnrealEndpointTooltip(EndpointInfo);
											return ToolTip;
										}
									)
									.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
							}
						)
						.OnSelectionChanged_Lambda(
							[this](TSharedPtr<FUnrealEndpointInfo> InEndpointInfo, ESelectInfo::Type InSelectType)
							{
								if (InEndpointInfo)
								{
									PropertyHandle->SetValue(InEndpointInfo->HostName);
								}
							}
						)
						.Content()
						[
							SNew(STextBlock)
								.MinDesiredWidth(100)
								.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
								.Text_Lambda(
									[this]() -> FText
									{
										if (TargetHostName)
										{
											return FText::FromString(*TargetHostName);
										}

										return {};
									}
								)
								.ColorAndOpacity_Lambda(
									[this]() -> FSlateColor {

										if (!TargetEndpointInfo)
										{
											// Target endpoint has not been discovered
											return FStyleColors::Warning;
										}

										return FStyleColors::Foreground;
									}
								)
								.ToolTipText_Lambda(
									[this]()
									{
										if (!TargetEndpointInfo)
										{
											// Target endpoint has not been discovered
											return FText::FromString(TEXT("Host is currently unavailable"));
										}

										FText ToolTip = GetUnrealEndpointTooltip(TargetEndpointInfo.ToSharedRef());
										return ToolTip;
									}
								)
						]
				]
		];
}



TArray<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>> SIngestSelectableUnrealEndpoint::GetLatestEndpointInfos()
{
	using namespace UE::CaptureManager;
	using namespace UE::CaptureManager::Private;

	TArray<TWeakPtr<FUnrealEndpoint>> WeakEndpoints = UnrealEndpointManager->GetEndpoints();

	TArray<FUnrealEndpointInfo> SortedEndpointInfos;
	SortedEndpointInfos.Reserve(WeakEndpoints.Num());

	for (const TWeakPtr<FUnrealEndpoint>& WeakEndpoint : WeakEndpoints)
	{
		if (const TSharedPtr<FUnrealEndpoint> Endpoint = WeakEndpoint.Pin())
		{
			FUnrealEndpointInfo EndpointInfo = Endpoint->GetInfo();
			SortedEndpointInfos.Emplace(EndpointInfo);
		}
	}

	// Keep the order deterministic
	SortedEndpointInfos.Sort(FUnrealEndpointInfoComparator());

	// Convert to shared for the combo box
	TArray<TSharedRef<FUnrealEndpointInfo>> SharedEndpointInfos;
	SharedEndpointInfos.Reserve(SortedEndpointInfos.Num());

	for (const FUnrealEndpointInfo& EndpointInfo : SortedEndpointInfos)
	{
		SharedEndpointInfos.Emplace(MakeShared<FUnrealEndpointInfo>(EndpointInfo));
	}

	return SharedEndpointInfos;
}

void SIngestSelectableUnrealEndpoint::SetEndpointInfos(TArray<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>> InEndpointInfos)
{
	EndpointInfos = MoveTemp(InEndpointInfos);
	UpdateTargetEndpointInfo();

	if (ComboBox)
	{
		ComboBox->RefreshOptions();
	}
}

void SIngestSelectableUnrealEndpoint::UpdateTargetEndpointInfo()
{
	using namespace UE::CaptureManager;

	TargetEndpointInfo = nullptr;

	if (!TargetHostName)
	{
		return;
	}

	const TSharedRef<FUnrealEndpointInfo>* Found = EndpointInfos.FindByPredicate(
		[this](const TSharedRef<FUnrealEndpointInfo>& InEndpointInfo)
		{
			return InEndpointInfo->HostName == *TargetHostName;
		}
	);

	if (Found)
	{
		TargetEndpointInfo = Found->ToSharedPtr();
	}
}

void SIngestSelectableUnrealEndpoint::OnPropertyChanged()
{
	using namespace UE::CaptureManager;

	FString HostName;
	FPropertyAccess::Result GetValueResult = PropertyHandle->GetValue(HostName);

	if (GetValueResult == FPropertyAccess::Result::Success)
	{
		TargetHostName = MakeShared<FString>(HostName);
	}
	else
	{
		TargetHostName = nullptr;
	}

	UpdateTargetEndpointInfo();
}

#undef LOCTEXT_NAMESPACE
