// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CaptureManagerUnrealEndpointManager.h"

#include "PropertyHandle.h"
#include "Widgets/Input/SComboBox.h"

class SIngestSelectableUnrealEndpoint : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SIngestSelectableUnrealEndpoint) {}
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
	SLATE_END_ARGS()

	SIngestSelectableUnrealEndpoint();
	~SIngestSelectableUnrealEndpoint();

	void Construct(const FArguments& InArgs);

private:
	TArray<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>> GetLatestEndpointInfos();
	void SetEndpointInfos(TArray<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>> InEndpointInfos);
	void UpdateTargetEndpointInfo();
	void OnPropertyChanged();

	TSharedRef<UE::CaptureManager::FUnrealEndpointManager> UnrealEndpointManager;
	TArray<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>> EndpointInfos;
	TSharedPtr<UE::CaptureManager::FUnrealEndpointInfo> TargetEndpointInfo;

	TSharedPtr<FString> TargetHostName;
	FString LocalHostName;
	TSharedPtr<SComboBox<TSharedRef<UE::CaptureManager::FUnrealEndpointInfo>>> ComboBox;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	FDelegateHandle EndpointsChangedDelegateHandle;
};
