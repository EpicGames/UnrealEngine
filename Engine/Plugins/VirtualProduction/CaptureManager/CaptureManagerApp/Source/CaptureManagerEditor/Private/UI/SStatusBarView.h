// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SWidgetDrawer.h"

class SStatusBarView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStatusBarView) { }
	SLATE_END_ARGS()

	virtual ~SStatusBarView() override;

	void Construct(const FArguments& InArgs, FName InStatusBarId);

	void UpdateConnectionState(bool InState);

private:
	TSharedRef<SWidgetDrawer> MakeWidgetDrawer(FName StatusBarId);

	FText GetConnectionStateText() const;
	FText GetStatusToolTip() const;
	FSlateColor GetStatusColor() const;

	TSharedPtr<SWidgetDrawer> WidgetDrawer;

	bool IsConnected;
};
