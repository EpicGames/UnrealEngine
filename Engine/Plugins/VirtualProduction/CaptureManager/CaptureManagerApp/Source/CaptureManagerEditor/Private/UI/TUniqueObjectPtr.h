// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "UObject/Object.h"
#include "UObject/StrongObjectPtrTemplates.h"

template<class T>
class TUniqueObjectPtr
{
public:

	static_assert(std::is_base_of_v<UObject, T>, "Provided pointer has to be derived from UObject");

	TUniqueObjectPtr()
		: Object(nullptr)
	{
	}

	TUniqueObjectPtr(T* InObject)
		: Object(InObject)
	{
	}

	~TUniqueObjectPtr()
	{
		if (IsValid() && Object.IsValid())
		{
			Object->MarkAsGarbage();

			Object = nullptr;
		}
	}

	TUniqueObjectPtr(const TUniqueObjectPtr<T>& InOther) = delete;
	TUniqueObjectPtr<T>& operator=(const TUniqueObjectPtr<T>& InOther) = delete;

	TUniqueObjectPtr(TUniqueObjectPtr<T>&& InOther) = default;
	TUniqueObjectPtr<T>& operator=(TUniqueObjectPtr<T>&& InOther) = default;

	const T& operator*() const
	{
		return *Object.Get();
	}

	T& operator*()
	{
		return *Object.Get();
	}

	const T* operator->() const
	{
		return Object.Get();
	}

	T* operator->()
	{
		return Object.Get();
	}

	T* Get()
	{
		return Object.Get();
	}

	const T* Get() const
	{
		return Object.Get();
	}

	bool IsValid() const
	{
		return Object != nullptr;
	}

private:

	TStrongObjectPtr<T> Object;
};