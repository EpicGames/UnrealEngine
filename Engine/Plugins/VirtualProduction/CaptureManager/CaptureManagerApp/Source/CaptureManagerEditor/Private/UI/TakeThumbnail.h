// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/GCObject.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/Texture2D.h"

class FTakeThumbnail : public FGCObject
{
public:
	FString ThumbnailPath;

	TSharedPtr<FSlateBrush> SlateBrush;
	TObjectPtr<class UTexture2D> Texture = nullptr;
	bool IsLoaded = false;

	FTakeThumbnail()
	{
		ThumbnailPath = "";
		Texture = nullptr;
		IsLoaded = false;
	}

	FTakeThumbnail(UTexture2D* InTexture)
	{
		Texture = InTexture;
		SlateBrush = MakeShared<FSlateImageBrush>((UObject*)Texture, FVector2D(Texture->GetSizeX(), Texture->GetSizeY()));
		IsLoaded = true;
	}

	virtual ~FTakeThumbnail()
	{
		SlateBrush = nullptr;
		Texture = nullptr;
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(Texture);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FTakeThumbnail");
	}
};
