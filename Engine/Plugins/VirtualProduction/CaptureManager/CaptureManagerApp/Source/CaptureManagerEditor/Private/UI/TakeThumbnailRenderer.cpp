// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakeThumbnailRenderer.h"
#include "ThumbnailHelpers.h"
#include "TakeVirtualAsset.h"
#include "Engine/Texture2D.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "TextureResource.h"
#include "CanvasTypes.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ImageUtils.h"

#define LOCTEXT_NAMESPACE "TakeThumbnailRenderer"

bool UTakeThumbnailRenderer::CanVisualizeAsset(UObject* InObject)
{
	if (UTakeVirtualAsset* Take = Cast<UTakeVirtualAsset>(InObject))
	{
		return Take->Thumbnail.IsLoaded;
	}

	return false;
}

void UTakeThumbnailRenderer::Draw(UObject* InObject, int32 InX, int32 InY, uint32 InWidth, uint32 InHeight, FRenderTarget* InRenderTarget, FCanvas* InCanvas, bool bInAdditionalViewFamily)
{
	if (UTakeVirtualAsset* Take = Cast<UTakeVirtualAsset>(InObject))
	{
		uint32 Width = Take->Thumbnail.Texture->GetSizeX();
		uint32 Height = Take->Thumbnail.Texture->GetSizeY();

		float WRatio = (float)Width / InWidth;
		float HRatio = (float)Height / InHeight;

		bool WidthFit = WRatio > HRatio;

		Width = WidthFit ? InWidth : Width / HRatio;
		Height = WidthFit ? Height / WRatio : InHeight;

		int32 Y = (InHeight - Height) / 2;
		int32 X = (InWidth - Width) / 2;

		InCanvas->DrawTile(X, Y, Width, Height, 0, 0, 1, 1, FLinearColor::White, Take->Thumbnail.Texture->GetResource());
	}
};

void UTakeThumbnailRenderer::BeginDestroy()
{
	Super::BeginDestroy();
}

#undef LOCTEXT_NAMESPACE
