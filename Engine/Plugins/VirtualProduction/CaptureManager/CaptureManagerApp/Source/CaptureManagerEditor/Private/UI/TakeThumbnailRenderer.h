// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"

#include "TakeThumbnailRenderer.generated.h"

class UObject;
class FRenderTarget;
class FCanvas;

UCLASS(MinimalAPI)
class UTakeThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

public:
	virtual bool CanVisualizeAsset(UObject* InObject) override;
	virtual void Draw(UObject* InObject, int32 InX, int32 InY, uint32 InWidth, uint32 InHeight, FRenderTarget* InRenderTarget, FCanvas* InCanvas, bool bInAdditionalViewFamily) override;
	virtual void BeginDestroy() override;
};
