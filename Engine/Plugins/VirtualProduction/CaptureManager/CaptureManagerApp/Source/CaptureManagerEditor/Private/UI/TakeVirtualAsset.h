// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TakeThumbnail.h"
#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "TakeVirtualAsset.generated.h"

UCLASS()
class UTakeVirtualAsset : public UObject
{
public:
	GENERATED_BODY()

	UTakeVirtualAsset() {}

	UTakeVirtualAsset(FGuid InCaptureDeviceId, UE::CaptureManager::FTakeId InTakeId, FTakeMetadata InMetadata)
		: CaptureDeviceId(InCaptureDeviceId)
		, TakeId(InTakeId)
		, Metadata(MoveTemp(InMetadata))
	{
	}

	FGuid CaptureDeviceId;
	UE::CaptureManager::FTakeId TakeId;
	FTakeMetadata Metadata;
	FTakeThumbnail Thumbnail;
};
