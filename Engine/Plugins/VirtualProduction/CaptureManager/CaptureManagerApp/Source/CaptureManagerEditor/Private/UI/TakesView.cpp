// Copyright Epic Games, Inc. All Rights Reserved.

#include "TakesView.h"

#include "CaptureManagerPanelController.h"
#include "SIngestJobProcessor.h"
#include "ImageUtils.h"
#include "CaptureSourceVirtualAsset.h"
#include "TakeThumbnail.h"
#include "ContentBrowserModule.h"
#include "Async/Monitor.h"
#include "Misc/FileHelper.h"
#include "ILiveLinkDeviceModule.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "SPositiveActionButton.h"
#include "Widgets/Images/SThrobber.h"
#include "Misc/MessageDialog.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "Ingest/IngestCapability_Events.h"
#include "Ingest/IngestCapability_TakeInformation.h"
#include "CaptureManagerConversionTypes.h"

#define LOCTEXT_NAMESPACE "CaptureManagerPanelViews"

DEFINE_LOG_CATEGORY_STATIC(LogTakesView, Log, All);

namespace UE::CaptureManager
{
template<typename ElementType, typename OtherElementType, typename Predicate>
bool EqualTo(const TArray<ElementType>& InFirstArray, const TArray<OtherElementType>& InSecondArray, Predicate InPredicate)
{
	if (InFirstArray.Num() != InSecondArray.Num())
	{
		return false;
	}

	for (int64 Index = 0; Index < InFirstArray.Num(); ++Index)
	{
		bool bMatchFound = false;
		for (int64 OtherIndex = 0; OtherIndex < InSecondArray.Num(); ++OtherIndex)
		{
			if (InPredicate(InFirstArray[Index], InSecondArray[OtherIndex]))
			{
				bMatchFound = true;
				break;
			}
		}

		if (!bMatchFound)
		{
			return false;
		}
	}

	return true;
}

static EImageRotation ToImageRotation(EMediaOrientation InOrientation)
{
	switch (InOrientation)
	{
		case EMediaOrientation::CW90:  return EImageRotation::CW_90;
		case EMediaOrientation::CW180: return EImageRotation::CW_180;
		case EMediaOrientation::CW270: return EImageRotation::CW_270;
		default: return EImageRotation::None;
	}
}
} // namespace UE::CaptureManager

static const FText JobPipelineText = LOCTEXT("JobPipelineText", "Pipeline");

static const FString OuterPackageName = TEXT("CaptureDevices");

STakesView::STakesView() = default;

void STakesView::Construct(
	const FArguments& InArgs,
	TSharedRef<UE::CaptureManager::FIngestPipelineManager> InIngestPipelineManager,
	TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> InIngestJobsSettingsManager,
	TUniquePtr<FGetCurrentSelectionDelegate> InGetCurrentSelectionDelegate
)
{
	GetCurrentSelectionDelegate = MoveTemp(InGetCurrentSelectionDelegate);
	check(GetCurrentSelectionDelegate);

	IngestPipelineManager = InIngestPipelineManager.ToSharedPtr();
	IngestJobSettingsManager = InIngestJobsSettingsManager.ToSharedPtr();
	OnAddTakesToIngestQueue = InArgs._OnAddTakesToIngestQueue;
	OnRefreshTakes = InArgs._OnRefreshTakes;
	CurrentPipeline = IngestPipelineManager->GetSelectedPipeline();

	const TArray<UE::CaptureManager::FPipelineDetails>& Pipelines = IngestPipelineManager->GetPipelines();
	for (const UE::CaptureManager::FPipelineDetails& Pipeline : Pipelines)
	{
		PipelineNames.Emplace(MakeShared<FText>(Pipeline.DisplayName));
	}

	RefreshButton = SNew(SPositiveActionButton)
		.Icon(FAppStyle::Get().GetBrush("Icons.Refresh"))
		.ToolTipText(LOCTEXT("RefreshTakes_Tooltip", "Refresh takes for the selected device"))
		.OnClicked(this, &STakesView::OnRefreshButtonClicked);

	AddToQueueButton = SNew(SPositiveActionButton)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddToQueueText", "Add to Queue"))
		.ToolTipText(LOCTEXT("AddToQueue_Tooltip", "Add selected takes to the ingest jobs list"))
		.OnClicked(this, &STakesView::OnAddToQueueButtonClicked);

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	static const FSlateBrush* CachedImage = FAppStyle::Get().GetBrush("UnrealCircle.Thick");

	FAssetPickerConfig TakesPickerConfig = InArgs._TakesPickerConfig;

	ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Bottom)
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.0f)
					.AutoWidth()
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						RefreshButton.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.Padding(2.0f)
					.AutoWidth()
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(STextBlock)
							.Text(JobPipelineText)
					]
					+ SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SComboBox<TSharedRef<FText>>)
						.OptionsSource(&PipelineNames)
						.OnGenerateWidget(this, &STakesView::OnGeneratePipelineNameWidget)
						.OnSelectionChanged(this, &STakesView::OnPipelineSelectionChanged)
						.Content()
						[
							SNew(STextBlock)
							.MinDesiredWidth(200)
							.Text_Lambda(
								[this]()
								{
									return CurrentPipeline.DisplayName;
								}
							)
							.ToolTipText_Lambda(
								[this]()
								{
									return CurrentPipeline.ToolTip;
								}
							)
						]
					]
					+ SHorizontalBox::Slot()
					.Padding(2.0f)
					.AutoWidth()
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						AddToQueueButton.ToSharedRef()
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						ContentBrowser.CreateAssetPicker(TakesPickerConfig)
					]
					+SOverlay::Slot()
					.VAlign(EVerticalAlignment::VAlign_Bottom)
					.HAlign(EHorizontalAlignment::HAlign_Right)
					[
						SAssignNew(LoadingBox, SHorizontalBox)
						.Visibility(EVisibility::Hidden)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(EVerticalAlignment::VAlign_Center)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("LoadingTakesText", "Detecting takes..."))
						]
						+ SHorizontalBox::Slot()
						.Padding(4.0)
						.AutoWidth()
						.VAlign(EVerticalAlignment::VAlign_Center)
						.HAlign(EHorizontalAlignment::HAlign_Center)
						[
							SNew(SCircularThrobber)
							.Period(0.3f)
							.Radius(10)
							.PieceImage(FCoreStyle::Get().GetBrush("Throbber.CircleChunk.Small"))
						]
					]
				]
		];
}

void STakesView::SetAddToQueueButtonEnabled(const bool bIsEnabled)
{
	check(AddToQueueButton);

	if (AddToQueueButton)
	{
		AddToQueueButton->SetEnabled(bIsEnabled);
	}
}

void STakesView::UpdateTakeListStarted()
{
	LoadingBox->SetVisibility(EVisibility::Visible);
	RefreshButton->SetEnabled(false);
}

void STakesView::UpdateTakeListFinished()
{
	LoadingBox->SetVisibility(EVisibility::Hidden);
	RefreshButton->SetEnabled(true);
}

TArray<TObjectPtr<UTakeVirtualAsset>> STakesView::GetSelectedTakeAssets() const
{
	check(GetCurrentSelectionDelegate);

	if (!GetCurrentSelectionDelegate)
	{
		return {};
	}

	const TArray<FAssetData> SelectedAssetDatas = GetCurrentSelectionDelegate->Execute();

	TArray<TObjectPtr<UTakeVirtualAsset>> TakeAssets;
	TakeAssets.Reserve(SelectedAssetDatas.Num());

	for (const FAssetData& AssetData : SelectedAssetDatas)
	{
		TObjectPtr<UTakeVirtualAsset> TakeAsset(Cast<UTakeVirtualAsset>(AssetData.GetAsset()));

		if (TakeAsset)
		{
			TakeAssets.Emplace(MoveTemp(TakeAsset));
		}
	}

	return TakeAssets;
}

FReply STakesView::OnRefreshButtonClicked()
{
	OnRefreshTakes.ExecuteIfBound();

	return FReply::Handled();
}

FReply STakesView::OnAddToQueueButtonClicked()
{
	check(IngestJobSettingsManager);

	if (IngestJobSettingsManager)
	{
		TArray<TObjectPtr<UTakeVirtualAsset>> SelectedTakeAssets = GetSelectedTakeAssets();

		// If the user has selected a lot of takes, rather than add them in a random order, we add them in the order of
		// acquisition, just so that it is deterministic
		SelectedTakeAssets.Sort(
			[](const UTakeVirtualAsset& LeftItem, const UTakeVirtualAsset& RightItem)
			{
				return LeftItem.Metadata.DateTime.Get(FDateTime()) < RightItem.Metadata.DateTime.Get(FDateTime());
			}
		);

		const TStrongObjectPtr<const UIngestJobSettings> DefaultSettings(NewObject<UIngestJobSettings>());

		if (DefaultSettings)
		{
			OnAddTakesToIngestQueue.ExecuteIfBound(MoveTemp(SelectedTakeAssets), *DefaultSettings);
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> STakesView::OnGeneratePipelineNameWidget(TSharedRef<FText> InText)
{
	check(IngestPipelineManager);

	if (TOptional<UE::CaptureManager::FPipelineDetails> MaybePipeline = IngestPipelineManager->GetPipelineByDisplayName(*InText))
	{
		UE::CaptureManager::FPipelineDetails Pipeline = MoveTemp(MaybePipeline.GetValue());
		FText Tooltip = Pipeline.ToolTip;
		FText DisplayName = Pipeline.DisplayName;

		return SNew(STextBlock)
			.Text(DisplayName)
			.ToolTipText(Tooltip);
	}

	check(false);
	return SNullWidget::NullWidget;
}

void STakesView::OnPipelineSelectionChanged(TSharedPtr<FText> InText, ESelectInfo::Type InSelectType)
{
	check(IngestPipelineManager);

	if (TOptional<UE::CaptureManager::FPipelineDetails> SelectedPipeline = IngestPipelineManager->SelectPipelineByDisplayName(*InText))
	{
		CurrentPipeline = SelectedPipeline.GetValue();
	}
}

TArray<FAssetData> FTakesView::CreateTakeAssetsDataForCaptureDevice(TOptional<FString> InCaptureDeviceName)
{
	using namespace UE::CaptureManager;

	TArray<TObjectPtr<UTakeVirtualAsset>> SelectedDevicesTakesArray;

	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();
	if (Controller)
	{
		for (TPair<FGuid, TArray<TUniqueObjectPtr<UTakeVirtualAsset>>>& DeviceIdTakeTuple : CaptureDeviceTakesMap)
		{
			FGuid CaptureDeviceId = DeviceIdTakeTuple.Key;
			TObjectPtr<ULiveLinkDevice> Device = Controller->GetCaptureDevice(CaptureDeviceId);

			if (!InCaptureDeviceName.IsSet() || Device.GetName() == InCaptureDeviceName.GetValue())
			{
				TArray<TObjectPtr<UTakeVirtualAsset>> Takes;

				for (TUniqueObjectPtr<UTakeVirtualAsset>& Take : DeviceIdTakeTuple.Value)
				{
					Takes.Add(TObjectPtr<UTakeVirtualAsset>(Take.Get()));
				}

				SelectedDevicesTakesArray.Append(MoveTemp(Takes));
			}
		}
	}

	return CreateTakeAssetsData(SelectedDevicesTakesArray);
}

TArray<FAssetData> FTakesView::CreateTakeAssetsData(const TArray<TObjectPtr<UTakeVirtualAsset>>& InSelectedCaptureDevicesTakesArray)
{
	TArray<FAssetData> TakeAssetsData;
	for (const TObjectPtr<UTakeVirtualAsset>& TakeItem : InSelectedCaptureDevicesTakesArray)
	{
		FAssetData TakeAssetData(TakeItem.Get());
		TakeAssetData.AssetClassPath = FTopLevelAssetPath(*(TakeItem->GetPackage()->GetName()), *TakeItem->GetName());

		TakeAssetsData.Add(TakeAssetData);
	}

	return TakeAssetsData;
}

FTakesView::FTakesView(TWeakPtr<FCaptureManagerPanelController> InController) :
	PanelController(InController)
{
	CreateTakesView();
}


void FTakesView::AddTakesToIngestQueue(const TArray<TObjectPtr<UTakeVirtualAsset>>& InTakeAssets, const UIngestJobSettings& InJobSettings)
{
	using namespace UE::CaptureManager;

	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();

	if (!Controller)
	{
		UE_LOGF(LogTakesView, Error, "Failed to add takes to the ingest queue, controller is not available");
		return;
	}

	TArray<TSharedRef<FIngestJob>> IngestJobs;
	IngestJobs.Reserve(InTakeAssets.Num());

	const FPipelineDetails Pipeline = Controller->GetIngestPipelineManager()->GetSelectedPipeline();
	for (const TObjectPtr<UTakeVirtualAsset>& TakeAsset : InTakeAssets)
	{
		EImageRotation ImageRotationToApply = InJobSettings.ImageRotation;
		if (!TakeAsset->Metadata.Video.IsEmpty() && TakeAsset->Metadata.Video[0].Orientation.IsSet())
		{
			ImageRotationToApply = ToImageRotation(UE::CaptureManager::ToUprightRotation(TakeAsset->Metadata.Video[0].Orientation.GetValue()));
		}

		// We convert the UObject based settings into something that the ingest job can truly own
		FIngestJob::FSettings IngestJobSettings =
		{
			.WorkingDirectory = InJobSettings.WorkingDirectory.Path,
			.DownloadFolder = InJobSettings.DownloadFolder.Path,
			.VideoSettings =
			{
				.Format = InJobSettings.ImageFormat,
				.FileNamePrefix = InJobSettings.ImageFileNamePrefix,
				.ImagePixelFormat = InJobSettings.ImagePixelFormat,
				.ImageRotation = ImageRotationToApply
			},
			.AudioSettings =
			{
				.Format = InJobSettings.AudioFormat,
				.FileNamePrefix = InJobSettings.AudioFileNamePrefix
			},
			.UploadHostName = InJobSettings.UploadHostName
		};

		IngestJobs.Emplace(MakeShared<FIngestJob>(TakeAsset->CaptureDeviceId, TakeAsset->TakeId, TakeAsset->Metadata, Pipeline.PipelineConfig, MoveTemp(IngestJobSettings)));
	}

	const int32 ExpectedNumAdded = IngestJobs.Num();

	TSharedRef<SIngestJobProcessor> IngestJobProcessor = Controller->GetIngestJobProcessorWidget();
	const int32 NumAdded = IngestJobProcessor->AddJobs(MoveTemp(IngestJobs));

	if (NumAdded != ExpectedNumAdded)
	{
		UE_LOGF(LogTakesView, Error, "Some ingest jobs were not added to the queue (%d out of %d added)", NumAdded, ExpectedNumAdded);
	}
}

void FTakesView::Refresh()
{
	using namespace UE::CaptureManager;

	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();
	if (Controller)
	{	
		bool bHasQueuedJobs = false;
		const uint32 JobsToCount = static_cast<uint32>(UE::CaptureManager::FIngestJob::EProcessingState::Pending)
								 | static_cast<uint32>(UE::CaptureManager::FIngestJob::EProcessingState::Running);

		TSharedRef<UE::CaptureManager::SIngestJobProcessor> IngestJobProcessor = Controller->GetIngestJobProcessorWidget();
		if (SelectedItem.IsValid() && SelectedItem->Implements<ULiveLinkDeviceCapability_Ingest>())
		{
			const FGuid DeviceId = SelectedItem->GetDeviceId();
			bHasQueuedJobs = IngestJobProcessor->CountQueuedDeviceJobs(DeviceId, JobsToCount) > 0;
		}
		else
		{

			for (const TObjectPtr<ULiveLinkDevice>& Device : Controller->GetCaptureDevices())
			{
				if (IngestJobProcessor->CountQueuedDeviceJobs(Device->GetDeviceId(), JobsToCount) > 0)
				{
					bHasQueuedJobs = true;
					break;
				}
			}
		}

		if (bHasQueuedJobs)
		{
			const FText ConfirmationTitle = LOCTEXT("ClearAffectedDeviceJobsTitle", "Remove queued takes?");
			const FText ConfirmationMessage = LOCTEXT("ClearAffectedDeviceJobsMessage", "Refreshing will remove takes queued for ingest and cancel in progress ingest jobs. Do you wish to proceed with the refresh?");
			EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::YesNo, ConfirmationMessage, ConfirmationTitle);
			if (Answer == EAppReturnType::No)
			{
				// Give up refreshing
				return;
			}
		}
	}

	if (SelectedItem.IsValid() && SelectedItem->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		RefreshSingleDevice(SelectedItem.Get());
	}
	else
	{
		RefreshAllDevices();
	}
}

void FTakesView::RefreshSingleDevice(TObjectPtr<ULiveLinkDevice> InDevice)
{
	ResetTakesCache(InDevice->GetDeviceId());

	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();

	if (!Controller)
	{
		return;
	}

	TakesTileView->UpdateTakeListStarted();

	FIngestUpdateTakeListCallback Callback =
		FIngestUpdateTakeListCallback::Type::CreateRaw(this, &FTakesView::UpdateTakeListCallback, InDevice->GetDeviceId());

	TSharedRef<FUpdateTakeListManager> UpdateTakeListManager = Controller->GetUpdateTakeListManager();
	UpdateTakeListManager->UpdateTakeListForDevice(InDevice->GetDeviceId(), MoveTemp(Callback));
}

void FTakesView::RefreshAllDevices()
{
	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();

	if (!Controller)
	{
		return;
	}

	for (const TObjectPtr<ULiveLinkDevice>& Device : Controller->GetCaptureDevices())
	{
		RefreshSingleDevice(Device);
	}
}

void FTakesView::UpdateTakeListCallback(TArray<UE::CaptureManager::FTakeId> InTakeIds, FGuid InCaptureDeviceId)
{
	using namespace UE::CaptureManager;

	check(IsInGameThread());

	TakesTileView->UpdateTakeListFinished();

	TArray<TUniqueObjectPtr<UTakeVirtualAsset>>* Found = CaptureDeviceTakesMap.Find(InCaptureDeviceId);
	if (!Found)
	{
		return;
	}

	const TArray<TUniqueObjectPtr<UTakeVirtualAsset>>& TakeAssets = *Found;

	if (TakeAssets.IsEmpty() && !InTakeIds.IsEmpty())
	{
		TArray<TUniqueObjectPtr<UTakeVirtualAsset>> TakeObjects = MakeTakeObjects(InCaptureDeviceId, InTakeIds);

		CaptureDeviceTakesMap.Add(InCaptureDeviceId, MoveTemp(TakeObjects));

		RefreshTakesViewDelegate.ExecuteIfBound(true);
		return;
	}
}

void FTakesView::ResetTakesCache(FGuid InCaptureDeviceId)
{
	if (CaptureDeviceTakesMap.Contains(InCaptureDeviceId))
	{
		TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();
		if (Controller)
		{
			TSharedRef<UE::CaptureManager::SIngestJobProcessor> IngestJobProcessor = Controller->GetIngestJobProcessorWidget();
			IngestJobProcessor->RemoveJobsForDevice(InCaptureDeviceId);
		}
		CaptureDeviceTakesMap[InCaptureDeviceId].Empty();
	}
	else
	{
		CaptureDeviceTakesMap.Add(InCaptureDeviceId);
	}
}

void FTakesView::CreateTakesView()
{
	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();

	if (!Controller)
	{
		return;
	}

	TSharedRef<UE::CaptureManager::FIngestPipelineManager> IngestPipelineManager = Controller->GetIngestPipelineManager();
	TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> IngestJobSettingsManager = Controller->GetIngestJobSettingsManager();

	TUniquePtr<FGetCurrentSelectionDelegate> GetCurrentSelectionDelegate = MakeUnique<FGetCurrentSelectionDelegate>();

	FAssetPickerConfig TakesPickerConfig;
	{
		TakesPickerConfig.SelectionMode = ESelectionMode::Multi;
		TakesPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
		TakesPickerConfig.bFocusSearchBoxWhenOpened = true;
		TakesPickerConfig.bAllowNullSelection = false;
		TakesPickerConfig.bShowBottomToolbar = true;
		TakesPickerConfig.bAutohideSearchBar = false;
		TakesPickerConfig.bAllowDragging = false;
		TakesPickerConfig.bCanShowClasses = false;
		TakesPickerConfig.bShowPathInColumnView = false;
		TakesPickerConfig.bSortByPathInColumnView = false;
		TakesPickerConfig.bShowTypeInColumnView = false;
		TakesPickerConfig.bForceShowEngineContent = true;
		TakesPickerConfig.bForceShowPluginContent = true;
		TakesPickerConfig.RefreshAssetViewDelegates.Add(&RefreshTakesViewDelegate);
		TakesPickerConfig.Filter.ClassPaths.Add(UTakeVirtualAsset::StaticClass()->GetClassPathName());
		TakesPickerConfig.Filter.bRecursiveClasses = true;
		TakesPickerConfig.Filter.bRecursivePaths = true;
		TakesPickerConfig.GetCurrentSelectionDelegates.Add(GetCurrentSelectionDelegate.Get());

		TakesPickerConfig.OnGetCustomSourceAssets =
			FOnGetCustomSourceAssets::CreateLambda([this](const FARFilter& SourceFilter, TArray<FAssetData>& OutAssets)
				{
					TOptional<FString> CaptureDeviceName;
					if (SelectedItem.IsValid())
					{
						CaptureDeviceName = SelectedItem->GetName();
					}

					OutAssets = CreateTakeAssetsDataForCaptureDevice(CaptureDeviceName);
				});
	}

	SAssignNew(TakesTileView, STakesView, MoveTemp(IngestPipelineManager), MoveTemp(IngestJobSettingsManager), MoveTemp(GetCurrentSelectionDelegate))
		.OnAddTakesToIngestQueue_Raw(this, &FTakesView::AddTakesToIngestQueue)
		.OnRefreshTakes_Raw(this, &FTakesView::Refresh)
		.TakesPickerConfig(TakesPickerConfig);
}

void FTakesView::CaptureDeviceStarted(FGuid InCaptureDeviceId)
{
	AsyncTask(ENamedThreads::GameThread, [this, Controller = PanelController.Pin(), InCaptureDeviceId]()
		{
			if (!Controller)
			{
				return;
			}

			using namespace UE::CaptureManager;

			TObjectPtr<ULiveLinkDevice> Device = Controller->GetCaptureDevice(InCaptureDeviceId);

			if (!Device)
			{
				return;
			}

			if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
			{
				return;
			}

			RefreshSingleDevice(Device);
		});
}

void FTakesView::CaptureDeviceStopped(FGuid InCaptureDeviceId)
{
	AsyncTask(ENamedThreads::GameThread, [this, Controller = PanelController.Pin(), InCaptureDeviceId]()
		{
			if (!Controller)
			{
				return;
			}

			using namespace UE::CaptureManager;

			TObjectPtr<ULiveLinkDevice> Device = Controller->GetCaptureDevice(InCaptureDeviceId);

			if (!Device)
			{
				return;
			}

			CaptureDeviceTakesMap.Remove(InCaptureDeviceId);
			RefreshTakesViewDelegate.ExecuteIfBound(true);
		});
}

void FTakesView::CaptureDeviceAdded(ULiveLinkDevice* InDevice)
{
	AsyncTask(ENamedThreads::GameThread, [this, Controller = PanelController.Pin(), Device = TStrongObjectPtr<ULiveLinkDevice>(InDevice)]()
	{
		if (!Controller)
		{
			return;
		}

		using namespace UE::CaptureManager;

		if (!Device)
		{
			return;
		}

		if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
		{
			return;
		}

		FString PackageName = GetTransientPackage()->GetPathName() / OuterPackageName;
		UPackage* Package = FindPackage(nullptr, *PackageName);
		if (!Package)
		{
			Package = CreatePackage(*PackageName);
		}

		TScriptInterface<ILiveLinkDeviceCapability_Ingest> CapabilityInterface(Device.Get());

		FGuid CaptureDeviceId = Device->GetDeviceId();

		CapabilityInterface->SubscribeToEvent(FString(FIngestCapability_TakeAddedEvent::Name),
			FCaptureEventHandler([this, CaptureDeviceId](TSharedPtr<const FCaptureEvent> InTakeAddedEvent)
				{
					check(IsInGameThread());

					TSharedPtr<const FIngestCapability_TakeAddedEvent> Event =
						StaticCastSharedPtr<const FIngestCapability_TakeAddedEvent>(InTakeAddedEvent);

					TArray<FTakeId> TakeIds = { Event->TakeId };
					TArray<TUniqueObjectPtr<UTakeVirtualAsset>> NewTakeObject = MakeTakeObjects(CaptureDeviceId, { TakeIds });

					if (NewTakeObject.IsEmpty())
					{
						return;
					}

					if (TArray<TUniqueObjectPtr<UTakeVirtualAsset>>* TakeObjects = CaptureDeviceTakesMap.Find(CaptureDeviceId))
					{
						TakeObjects->Append(MoveTemp(NewTakeObject));

						RefreshTakesViewDelegate.ExecuteIfBound(true);
					}
				}));

		CapabilityInterface->SubscribeToEvent(FString(FIngestCapability_TakeUpdatedEvent::Name),
			FCaptureEventHandler([this, CaptureDeviceId, CapabilityInterface](TSharedPtr<const FCaptureEvent> InTakeUpdatedEvent)
				{
					check(IsInGameThread());

					TSharedPtr<const FIngestCapability_TakeUpdatedEvent> Event =
						StaticCastSharedPtr<const FIngestCapability_TakeUpdatedEvent>(InTakeUpdatedEvent);

					auto FindPredicate = 
						[TakeId = Event->TakeId](const TUniqueObjectPtr<UTakeVirtualAsset>& InTakeObject)
						{
							return InTakeObject->TakeId == TakeId;
						};

					TArray<TUniqueObjectPtr<UTakeVirtualAsset>>* MaybeTakeObjects = CaptureDeviceTakesMap.Find(CaptureDeviceId);
					if (!MaybeTakeObjects)
					{
						return;
					}

					TUniqueObjectPtr<UTakeVirtualAsset>* MaybeTakeObject = MaybeTakeObjects->FindByPredicate(FindPredicate);
					if (!MaybeTakeObject)
					{
						return;
					}

					TUniqueObjectPtr<UTakeVirtualAsset>& TakeObject = *MaybeTakeObject;

					TOptional<FTakeMetadata> TakeInfoOpt = CapabilityInterface->GetTakeMetadata(Event->TakeId);
					if (!TakeInfoOpt.IsSet())
					{
						return;
					}

					FTakeMetadata TakeInfo = TakeInfoOpt.GetValue();

					TakeObject->CaptureDeviceId = CaptureDeviceId;
					TakeObject->TakeId = Event->TakeId;
					TakeObject->Metadata = MoveTemp(TakeInfo);

					TOptional<FTakeThumbnail> TakeThumbnail = CreateThumbnail(TakeObject->Metadata.Thumbnail);

					if (TakeThumbnail.IsSet())
					{
						TakeObject->Thumbnail = MoveTemp(TakeThumbnail.GetValue());
					}

					RefreshTakesViewDelegate.ExecuteIfBound(true);
				}));

		CapabilityInterface->SubscribeToEvent(FString(FIngestCapability_TakeRemovedEvent::Name),
			FCaptureEventHandler([this, CaptureDeviceId](TSharedPtr<const FCaptureEvent> InTakeRemovedEvent)
				{
					check(IsInGameThread());

					TSharedPtr<const FIngestCapability_TakeRemovedEvent> Event =
						StaticCastSharedPtr<const FIngestCapability_TakeRemovedEvent>(InTakeRemovedEvent);

					if (TArray<TUniqueObjectPtr<UTakeVirtualAsset>>* TakeObjects = CaptureDeviceTakesMap.Find(CaptureDeviceId))
					{
						int32 NumRemoved = TakeObjects->RemoveAll([this, TakeId = Event->TakeId](const TUniqueObjectPtr<UTakeVirtualAsset>& InObject)
							{
								return InObject->TakeId == TakeId;
							});

						if (NumRemoved > 0)
						{
							RefreshTakesViewDelegate.ExecuteIfBound(true);
						}
					}
				}));
	});
}

void FTakesView::CaptureDeviceRemoved(ULiveLinkDevice* InDevice)
{
	// ULiveLinkDevice has been removed so handle device removal in Capture Manager (e.g. removal of ingest jobs)
	AsyncTask(ENamedThreads::GameThread, [this, Controller = PanelController.Pin(), Device = TStrongObjectPtr<ULiveLinkDevice>(InDevice)]()
		{
			TScriptInterface<ILiveLinkDeviceCapability_Ingest> CapabilityInterface(Device.Get());

			CapabilityInterface->UnsubscribeAll();

			TSharedRef<UE::CaptureManager::SIngestJobProcessor> IngestJobProcessor = Controller->GetIngestJobProcessorWidget();

			// There's a period between RunIngest and the assignment of the member context, during which Stop() may have
			// been called. If so, immediately terminate (as Cancel() will not be called otherwise).
			IngestJobProcessor->Stop(Device->GetDeviceId());
			

			int32 NumJobsRemoved = IngestJobProcessor->RemoveJobsForDevice(Device->GetDeviceId());
			if (NumJobsRemoved > 0)
			{
				UE_LOGF(LogTakesView, Display, "Device '%ls' removed. Removed %d corresponding ingest jobs from the queue.", *Device->GetDisplayName().ToString(), NumJobsRemoved);
			}
			
			CaptureDeviceTakesMap.Remove(Device->GetDeviceId());
			RefreshTakesViewDelegate.ExecuteIfBound(true);
		});
}

void FTakesView::DeviceSelectionChanged(ULiveLinkDevice* InDevice)
{
	SelectedItem = InDevice;
	RefreshTakesViewDelegate.ExecuteIfBound(true);
}

TArray<TUniqueObjectPtr<UTakeVirtualAsset>> FTakesView::MakeTakeObjects(FGuid InCaptureDeviceId, TOptional<TArray<UE::CaptureManager::FTakeId>> InTakeIds)
{
	TArray<TUniqueObjectPtr<UTakeVirtualAsset>> TakeObjects;

	TSharedPtr<FCaptureManagerPanelController> Controller = PanelController.Pin();
	if (!Controller)
	{
		return TakeObjects;
	}

	using namespace UE::CaptureManager;

	TObjectPtr<ULiveLinkDevice> Device = Controller->GetCaptureDevice(InCaptureDeviceId);

	if (!Device)
	{
		return TakeObjects;
	}

	if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return TakeObjects;
	}

	const FString& CaptureDeviceName = Device.GetName();

	TScriptInterface<ILiveLinkDeviceCapability_Ingest> IngestInterface(Device);

	TArray<FTakeId> TakeIds;
	if (InTakeIds.IsSet())
	{
		TakeIds = InTakeIds.GetValue();
	}
	else
	{
		TakeIds = IngestInterface->GetTakeIdentifiers();
	}

	FString PackageName = GetTransientPackage()->GetPathName() / OuterPackageName / CaptureDeviceName;
	UPackage* Package = FindPackage(nullptr, *PackageName);
	if (!Package)
	{
		Package = CreatePackage(*PackageName);
	}

	for (const FTakeId& TakeId : TakeIds)
	{
		TOptional<FTakeMetadata> TakeInfoOpt = IngestInterface->GetTakeMetadata(TakeId);
		if (!TakeInfoOpt.IsSet())
		{
			continue;
		}

		FTakeMetadata TakeInfo = TakeInfoOpt.GetValue();

		FName Name = MakeUniqueTakeName(TakeInfo, Package);
		TUniqueObjectPtr<UTakeVirtualAsset> TakeObject(NewObject<UTakeVirtualAsset>(Package, Name));

		TakeObject->CaptureDeviceId = InCaptureDeviceId;
		TakeObject->TakeId = TakeId;
		TakeObject->Metadata = MoveTemp(TakeInfo);

		TOptional<FTakeThumbnail> TakeThumbnail = CreateThumbnail(TakeObject->Metadata.Thumbnail);

		if (TakeThumbnail.IsSet())
		{
			TakeObject->Thumbnail = MoveTemp(TakeThumbnail.GetValue());
		}

		TakeObjects.Add(MoveTemp(TakeObject));
	}

	return TakeObjects;
}

TOptional<FTakeThumbnail> FTakesView::CreateThumbnail(const FTakeThumbnailData& InThumbnailData)
{
	TOptional<TArray<uint8>> ThumbnailRawDataOpt = InThumbnailData.GetThumbnailData();
	TOptional<FTakeThumbnailData::FRawImage> ThumbnailRawImageOpt = InThumbnailData.GetRawImage();

	UTexture2D* PreviewImageTexture = nullptr; 
	if (ThumbnailRawDataOpt.IsSet())
	{
		FImage Image;
		TArray<uint8> ThumbnailRawData = MoveTemp(ThumbnailRawDataOpt.GetValue());
		FImageUtils::DecompressImage(ThumbnailRawData.GetData(), ThumbnailRawData.Num(), Image);
		Image.ChangeFormat(ERawImageFormat::BGRA8, EGammaSpace::sRGB);

		PreviewImageTexture = FImageUtils::CreateTexture2DFromImage(Image);
	}
	else if (ThumbnailRawImageOpt.IsSet())
	{
		FTakeThumbnailData::FRawImage RawImage = MoveTemp(ThumbnailRawImageOpt.GetValue());

		FImageView ImageView(RawImage.DecompressedImageData.GetData(), RawImage.Width, RawImage.Height);

		PreviewImageTexture = FImageUtils::CreateTexture2DFromImage(ImageView);
	}

	if (!PreviewImageTexture)
	{
		return {};
	}

	return FTakeThumbnail(PreviewImageTexture);
}

FName FTakesView::MakeUniqueTakeName(FTakeMetadata TakeInfo, UPackage* Package)
{
	FName UniqueName(FString::Format(TEXT("{0}_{1}"), { TakeInfo.Slate, TakeInfo.TakeNumber }));
	UObject* FoundObject = StaticFindObjectFastInternal(nullptr, Package, UniqueName);

	int32 UniquePartIncrement = 0;
	while (IsValid(FoundObject))
	{
		FName MaybeUniqueName(FString::Format(TEXT("{0}_{1}_{2}"), { TakeInfo.Slate, TakeInfo.TakeNumber, ++UniquePartIncrement }));
		FoundObject = StaticFindObjectFastInternal(nullptr, Package, MaybeUniqueName);
		UniqueName = MaybeUniqueName;
	}
	return UniqueName;
}

#undef LOCTEXT_NAMESPACE
