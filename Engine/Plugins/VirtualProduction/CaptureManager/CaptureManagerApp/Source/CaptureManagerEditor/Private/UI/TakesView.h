// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STileView.h"
#include "TakeVirtualAsset.h"
#include "CaptureSourceVirtualAsset.h"
#include "ContentBrowserDelegates.h"
#include "IContentBrowserSingleton.h"

#include "TUniqueObjectPtr.h"

#include "IngestManagement/IngestJobSettingsManager.h"
#include "IngestManagement/IngestPipelineManager.h"

class FCaptureManagerPanelController;
class FCaptureSourcesView;
class SEditableTextBox;
class SPositiveActionButton;
class SSearchBox;
class SSlateBoxBrush;
class ULiveLinkDevice;

namespace UE::CaptureManager
{
class SIngestJobProcessor;
}

class STakesView : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnAddTakesToIngestQueue, const TArray<TObjectPtr<UTakeVirtualAsset>>&, const UIngestJobSettings&);
	DECLARE_DELEGATE(FOnRefreshTakes);

	SLATE_BEGIN_ARGS(STakesView) {}
		SLATE_ARGUMENT(FAssetPickerConfig, TakesPickerConfig)
		SLATE_EVENT(FOnAddTakesToIngestQueue, OnAddTakesToIngestQueue)
		SLATE_EVENT(FOnRefreshTakes, OnRefreshTakes)
	SLATE_END_ARGS()

	STakesView();
	~STakesView() = default;

	void Construct(
		const FArguments& InArgs,
		TSharedRef<UE::CaptureManager::FIngestPipelineManager> InIngestPipelineManager,
		TSharedRef<UE::CaptureManager::FIngestJobSettingsManager> IngestJobsSettingsManager,
		TUniquePtr<FGetCurrentSelectionDelegate> GetCurrentSelectionDelegate
	);
	void SetAddToQueueButtonEnabled(bool bIsEnabled);
	void UpdateTakeListStarted();
	void UpdateTakeListFinished();

private:
	FReply OnRefreshButtonClicked();
	FReply OnAddToQueueButtonClicked();
	TSharedRef<SWidget> OnGeneratePipelineNameWidget(TSharedRef<FText> InText);
	void OnPipelineSelectionChanged(TSharedPtr<FText> InText, ESelectInfo::Type InSelectType);
	TArray<TObjectPtr<UTakeVirtualAsset>> GetSelectedTakeAssets() const;

	TUniquePtr<FGetCurrentSelectionDelegate> GetCurrentSelectionDelegate;
	TSharedPtr<SPositiveActionButton> RefreshButton;
	FOnRefreshTakes OnRefreshTakes;
	TSharedPtr<SPositiveActionButton> AddToQueueButton;
	FOnAddTakesToIngestQueue OnAddTakesToIngestQueue;
	TSharedPtr<SHorizontalBox> LoadingBox;

	TSharedPtr<UE::CaptureManager::FIngestPipelineManager> IngestPipelineManager;
	TArray<TSharedRef<FText>> PipelineNames;
	UE::CaptureManager::FPipelineDetails CurrentPipeline;
	TSharedPtr<UE::CaptureManager::FIngestJobSettingsManager> IngestJobSettingsManager;
};

class FTakesView
{
public:
	FTakesView(TWeakPtr<FCaptureManagerPanelController> InController);

	void CaptureDeviceStarted(FGuid InCaptureDeviceId);
	void CaptureDeviceStopped(FGuid InCaptureDeviceId);
	void CaptureDeviceAdded(ULiveLinkDevice* InDevice);
	void CaptureDeviceRemoved(ULiveLinkDevice* InDevice);
	void DeviceSelectionChanged(ULiveLinkDevice* InDevice);

	TSharedPtr<STakesView> TakesTileView;
private:
	void CreateTakesView();
	TWeakPtr<FCaptureManagerPanelController> PanelController;

	TMap<FGuid, TArray<TUniqueObjectPtr<UTakeVirtualAsset>>> CaptureDeviceTakesMap;
	TWeakObjectPtr<ULiveLinkDevice> SelectedItem;

	FRefreshAssetViewDelegate RefreshTakesViewDelegate;
	FSyncToAssetsDelegate SyncToDevicesDelegate;

	TArray<FAssetData> CreateTakeAssetsDataForCaptureDevice(TOptional<FString> InCaptureDeviceName = TOptional<FString>());
	TArray<FAssetData> CreateTakeAssetsData(const TArray<TObjectPtr<UTakeVirtualAsset>>& InSelectedCaptureDevicesTakesArray);

	void AddTakesToIngestQueue(const TArray<TObjectPtr<UTakeVirtualAsset>>& InTakeAssets, const UIngestJobSettings& InJobPreset);
	void Refresh();
	void RefreshSingleDevice(TObjectPtr<ULiveLinkDevice> InDevice);
	void RefreshAllDevices();
	void UpdateTakeListCallback(TArray<UE::CaptureManager::FTakeId> InTakeIds, FGuid InCaptureDeviceId);
	void ResetTakesCache(FGuid InCaptureDeviceId);

	TArray<TUniqueObjectPtr<UTakeVirtualAsset>> MakeTakeObjects(FGuid InCaptureDeviceId,
																TOptional<TArray<UE::CaptureManager::FTakeId>> InTakeIds = TOptional<TArray<UE::CaptureManager::FTakeId>>());
	TOptional<FTakeThumbnail> CreateThumbnail(const FTakeThumbnailData& InThumbnailData);

	FName MakeUniqueTakeName(FTakeMetadata TakeInfo, UPackage* Package);
};
