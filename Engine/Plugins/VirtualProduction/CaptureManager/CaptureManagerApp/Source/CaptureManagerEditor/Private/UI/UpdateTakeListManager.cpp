// Copyright Epic Games, Inc. All Rights Reserved.

#include "UpdateTakeListManager.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "Engine/Engine.h"

TSharedPtr<FUpdateTakeListManager> FUpdateTakeListManager::Create()
{
	TSharedPtr<FUpdateTakeListManager> Manager = 
		MakeShared<FUpdateTakeListManager>(FPrivate());

	return Manager;
}

FUpdateTakeListManager::FUpdateTakeListManager(FPrivate)
	: QueueRunner(UE::CaptureManager::TQueueRunner<FQueueRunnerContext>::FOnProcess::CreateStatic(&FUpdateTakeListManager::UpdateTakeListForDevice_Private))
{
}

FUpdateTakeListManager::~FUpdateTakeListManager()
{
	QueueRunner.Empty();
}


void FUpdateTakeListManager::UpdateTakeListForDevice(FGuid InDeviceId, FIngestUpdateTakeListCallback InCallback)
{
	FQueueRunnerContext Context;
	Context.DeviceId = MoveTemp(InDeviceId);
	Context.Callback = MoveTemp(InCallback);

	QueueRunner.Add(MoveTemp(Context));
}

void FUpdateTakeListManager::UpdateTakeListForDevice_Private(FQueueRunnerContext InContext)
{

	const FGuid& DeviceId = InContext.DeviceId;

	if (!DeviceId.IsValid())
	{
		return;
	}

	if (!GEngine)
	{
		return;
	}

	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	if (!Subsystem)
	{
		return;
	}

	const TObjectPtr<ULiveLinkDevice>* DevicePtr = Subsystem->GetDeviceMap().Find(DeviceId);
	if (!DevicePtr)
	{
		return;
	}

	TObjectPtr<ULiveLinkDevice> Device = *DevicePtr;
	if (!Device)
	{
		return;
	}

	if (!Device->Implements<ULiveLinkDeviceCapability_Ingest>())
	{
		return;
	}

	UIngestCapability_UpdateTakeListCallback* UpdateTakeListCallback = NewObject<UIngestCapability_UpdateTakeListCallback>();
	UpdateTakeListCallback->Callback = MoveTemp(InContext.Callback);

	ILiveLinkDeviceCapability_Ingest::Execute_UpdateTakeList(Device, UpdateTakeListCallback);
}
