// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/QueueRunner.h"
#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"

#include "LiveLinkDevice.h"
#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "Ingest/IngestCapability_UpdateTakeList.h"

class FUpdateTakeListManager
{
private:

	struct FPrivate { explicit FPrivate() = default; };

public:

	static TSharedPtr<FUpdateTakeListManager> Create();

	FUpdateTakeListManager(FPrivate InToken);
	~FUpdateTakeListManager();

	void UpdateTakeListForDevice(FGuid InDeviceId, FIngestUpdateTakeListCallback InCallback);

private:

	struct FQueueRunnerContext
	{
		FGuid DeviceId;
		FIngestUpdateTakeListCallback Callback;
	};

	static void UpdateTakeListForDevice_Private(FQueueRunnerContext InContext);

	UE::CaptureManager::TQueueRunner<FQueueRunnerContext> QueueRunner;
};