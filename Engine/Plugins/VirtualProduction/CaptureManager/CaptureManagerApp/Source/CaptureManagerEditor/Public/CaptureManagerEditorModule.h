// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "LiveLinkHubApplicationMode.h"

class FLiveLinkHubCaptureManagerMode;

class CAPTUREMANAGEREDITOR_API FCaptureManagerEditorModule : public IModuleInterface, public ILiveLinkHubApplicationModeFactory
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ Begin ILiveLinkHubApplicationModeFactory
	virtual TSharedRef<FLiveLinkHubApplicationMode> CreateLiveLinkHubAppMode(TSharedPtr<class FLiveLinkHubApplicationBase> InApp) override;
	//~ End ILiveLinkHubApplicationModeFactory
};
