// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CaptureManagerSettings : ModuleRules
{
	public CaptureManagerSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"NamingTokens",
			"PropertyEditor",
			"Slate",
			"SlateCore",
			"Settings",
			"UnrealEd",
			"CaptureUtils",
			"CaptureDataConverter"
		});
	}
}
