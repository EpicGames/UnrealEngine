// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerSettingsModule.h"

#include "ISettingsModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Settings/CaptureManagerSettings.h"
#include "Settings/CaptureManagerSettingsCustomization.h"
#include "CaptureManagerEncoderConfig.h"

#define LOCTEXT_NAMESPACE "CaptureManagerSettings"

namespace UE::CaptureManager::Private
{

// Populate empty encoder command fields with defaults and migrate pre-CL-40536694 placeholder names.
static void InitializeEncoderCommandDefaults(UCaptureManagerSettings* InSettings)
{
	if (InSettings->CustomVideoCommandArguments.IsEmpty())
	{
		InSettings->CustomVideoCommandArguments = FString(EncoderDefaults::VideoCommandArgs);
	}

	if (InSettings->CustomAudioCommandArguments.IsEmpty())
	{
		InSettings->CustomAudioCommandArguments = FString(EncoderDefaults::AudioCommandArgs);
	}

	// CL 40536694 renamed placeholders from {InputFileName}/{OutputFileName}/{ConversionFilters}
	// to {input}/{output}/{params}. Migrate any stale saved values.
	InSettings->CustomVideoCommandArguments.ReplaceInline(TEXT("{InputFileName}"), TEXT("{input}"));
	InSettings->CustomVideoCommandArguments.ReplaceInline(TEXT("{OutputFileName}"), TEXT("{output}"));
	InSettings->CustomVideoCommandArguments.ReplaceInline(TEXT("{ConversionFilters}"), TEXT("{params}"));

	InSettings->CustomAudioCommandArguments.ReplaceInline(TEXT("{InputFileName}"), TEXT("{input}"));
	InSettings->CustomAudioCommandArguments.ReplaceInline(TEXT("{OutputFileName}"), TEXT("{output}"));
}

} // namespace UE::CaptureManager::Private

void FCaptureManagerSettingsModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FCaptureManagerSettingsModule::PostEngineInit);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FCaptureManagerSettingsModule::EnginePreExit);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UCaptureManagerSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FCaptureManagerSettingsCustomization::MakeInstance));

	UE::CaptureManager::Private::InitializeEncoderCommandDefaults(GetMutableDefault<UCaptureManagerSettings>());
}

void FCaptureManagerSettingsModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FCaptureManagerSettingsModule::PostEngineInit()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		UCaptureManagerSettings* Settings = GetMutableDefault<UCaptureManagerSettings>();
		check(Settings);

		// Sanity check that the default upload host name is not empty
		check(!Settings->DefaultUploadHostName.IsEmpty());

		SettingsModule->RegisterSettings("Project", "Plugins", "Capture Manager",
			LOCTEXT("CaptureManagerSettingsName", "Capture Manager"),
			LOCTEXT("CaptureManagerDescription", "Configure Capture Manager."),
			Settings
		);
	}
}

void FCaptureManagerSettingsModule::EnginePreExit()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Capture Manager");
	}
}

IMPLEMENT_MODULE(FCaptureManagerSettingsModule, CaptureManagerSettings)

#undef LOCTEXT_NAMESPACE