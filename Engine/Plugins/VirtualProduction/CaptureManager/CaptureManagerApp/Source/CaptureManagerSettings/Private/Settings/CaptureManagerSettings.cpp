// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/CaptureManagerSettings.h"
#include "Settings/CaptureManagerTemplateTokens.h"
#include "CaptureManagerEncoderNamingTokens.h"
#include "Network/NetworkMisc.h"
#include "Misc/Paths.h"

#include "Async/TaskGraphInterfaces.h"

namespace UE::CaptureManager::Private
{
constexpr FStringView DefaultWorkingDirectory = TEXTVIEW("CaptureManager/Working/{yyyy}{mm}{dd}/{device}");
constexpr FStringView DefaultDownloadDirectory = TEXTVIEW("CaptureManager/Download/{yyyy}{mm}{dd}/{device}");
}

UCaptureManagerSettings::UCaptureManagerSettings(const FObjectInitializer& InObjectInitializer) :
	Super(InObjectInitializer)
{
	LocalHostName = UE::CaptureManager::GetLocalHostNameChecked();

	InitializeValuesIfNotSet();

	GeneralNamingTokens = InObjectInitializer.CreateDefaultSubobject<UCaptureManagerGeneralTokens>(this, TEXT("GeneralNamingTokens"));
	GeneralNamingTokens->CreateDefaultTokens();

	VideoEncoderNamingTokens = InObjectInitializer.CreateDefaultSubobject<UCaptureManagerVideoEncoderTokens>(this, TEXT("VideoEncoderNamingTokens"));
	VideoEncoderNamingTokens->CreateDefaultTokens();

	AudioEncoderNamingTokens = InObjectInitializer.CreateDefaultSubobject<UCaptureManagerAudioEncoderTokens>(this, TEXT("AudioEncoderNamingTokens"));
	AudioEncoderNamingTokens->CreateDefaultTokens();
}

TObjectPtr<UCaptureManagerGeneralTokens> UCaptureManagerSettings::GetGeneralNamingTokens() const
{
	return GeneralNamingTokens;
}

TObjectPtr<UCaptureManagerVideoEncoderTokens> UCaptureManagerSettings::GetVideoEncoderNamingTokens() const
{
	return VideoEncoderNamingTokens;
}

TObjectPtr<UCaptureManagerAudioEncoderTokens> UCaptureManagerSettings::GetAudioEncoderNamingTokens() const
{
	return AudioEncoderNamingTokens;
}

void UCaptureManagerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	InitializeValuesIfNotSet();
}

void UCaptureManagerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	FProperty* Property = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UCaptureManagerSettings, NumIngestExecutors));
	check(Property);

	FString MaxNumberOfExecutorsString = Property->GetMetaData(TEXT("ClampMax"));
	int32 MaxNumberOfExecutors;
	TTypeFromString<int32>::FromString(MaxNumberOfExecutors, *MaxNumberOfExecutorsString);

	NumIngestExecutors = FMath::Min(NumIngestExecutors, MaxNumberOfExecutors);
}

void UCaptureManagerSettings::InitializeValuesIfNotSet()
{
	using namespace UE::CaptureManager;

	if (DefaultWorkingDirectory.Path.IsEmpty())
	{
		DefaultWorkingDirectory.Path = FPaths::Combine(FPlatformProcess::UserSettingsDir(), Private::DefaultWorkingDirectory);
	}

	if (DownloadDirectory.Path.IsEmpty())
	{
		DownloadDirectory.Path = FPaths::Combine(FPlatformProcess::UserSettingsDir(), Private::DefaultDownloadDirectory);
	}

	if (DefaultUploadHostName.IsEmpty())
	{
		DefaultUploadHostName = LocalHostName;
	}
}