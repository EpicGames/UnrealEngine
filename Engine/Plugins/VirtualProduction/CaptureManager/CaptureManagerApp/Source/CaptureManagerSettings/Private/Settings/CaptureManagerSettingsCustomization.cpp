// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerSettingsCustomization.h"

#include "Settings/CaptureManagerTemplateTokens.h"
#include "Settings/CaptureManagerSettings.h"
#include "CaptureManagerEncoderNamingTokens.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Engine.h"
#include "NamingTokensEngineSubsystem.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CaptureManagerSettingsCustomization"

TSharedRef<IDetailCustomization> FCaptureManagerSettingsCustomization::MakeInstance()
{
	return MakeShared<FCaptureManagerSettingsCustomization>();
}

void FCaptureManagerSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	// Update current value when opening the settings page. Not safe to do under the settings object since there isn't
	// an explicit callback when the settings page is opened, and using a method like PostInitProperties fires on the CDO
	// early in the startup process.

	auto BuildSlate = [this](const TSharedRef<IPropertyHandle> InHandle, IDetailPropertyRow* InPropertyRow, const FNamingTokenFilterArgs& InTokenArgs) {
		InPropertyRow->CustomWidget()
			.WholeRowContent()
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0.f, 4.f)
					.AutoHeight()
					[
						SNew(STextBlock)
							.Text(InHandle->GetPropertyDisplayName())
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SEditableText)
							.IsReadOnly(true)
							.Text_Raw(this, &FCaptureManagerSettingsCustomization::GetDisplayTokenText, InTokenArgs)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
		};

	FNamingTokenFilterArgs BaseArgs =
	{
		.AdditionalNamespacesToInclude = {},
		.bIncludeGlobal = false
	};

	// General tokens
	const TSharedRef<IPropertyHandle> GeneralTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerSettings, GeneralTokens));
	IDetailPropertyRow* GeneralTokensRow = InDetailBuilder.EditDefaultProperty(GeneralTokensHandle);
	check(GeneralTokensRow);

	FNamingTokenFilterArgs GeneralArgs = BaseArgs;
	if (const TObjectPtr<const UCaptureManagerGeneralTokens> Tokens = GetDefault<UCaptureManagerSettings>()->GetGeneralNamingTokens())
	{
		GeneralArgs.AdditionalNamespacesToInclude.Add(Tokens->GetNamespace());
	}
	BuildSlate(GeneralTokensHandle, GeneralTokensRow, GeneralArgs);

	// Video encoder tokens
	const TSharedRef<IPropertyHandle> VideoTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerSettings, VideoCommandTokens));
	IDetailPropertyRow* VideoTokensRow = InDetailBuilder.EditDefaultProperty(VideoTokensHandle);
	check(VideoTokensRow);
	
	FNamingTokenFilterArgs VideoArgs = BaseArgs;
	if (const TObjectPtr<const UCaptureManagerVideoEncoderTokens> VideoTokens = GetDefault<UCaptureManagerSettings>()->GetVideoEncoderNamingTokens())
	{
		VideoArgs.AdditionalNamespacesToInclude.Add(VideoTokens->GetNamespace());
	}
	BuildSlate(VideoTokensHandle, VideoTokensRow, VideoArgs);
	
	// Audio encoder tokens
	const TSharedRef<IPropertyHandle> AudioTokensHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCaptureManagerSettings, AudioCommandTokens));
	IDetailPropertyRow* AudioTokensRow = InDetailBuilder.EditDefaultProperty(AudioTokensHandle);
	check(AudioTokensRow);
	
	FNamingTokenFilterArgs AudioArgs = BaseArgs;
	if (const TObjectPtr<const UCaptureManagerAudioEncoderTokens> AudioTokens = GetDefault<UCaptureManagerSettings>()->GetAudioEncoderNamingTokens())
	{
		AudioArgs.AdditionalNamespacesToInclude.Add(AudioTokens->GetNamespace());
	}
	BuildSlate(AudioTokensHandle, AudioTokensRow, AudioArgs);
}

FText FCaptureManagerSettingsCustomization::GetDisplayTokenText(FNamingTokenFilterArgs InArgs) const
{
	FString FormattedTokensString = TEXT("None");
	if (GEngine)
	{
		FormattedTokensString = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->GetFormattedTokensStringForDisplay(MoveTemp(InArgs));
	}
	return FText::FromString(FormattedTokensString);
}

#undef LOCTEXT_NAMESPACE