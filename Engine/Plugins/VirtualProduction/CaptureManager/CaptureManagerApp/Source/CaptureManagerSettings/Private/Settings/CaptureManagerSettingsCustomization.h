// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

struct FNamingTokenFilterArgs;

/**
 * Customization for the capture manager settings.
 */
class FCaptureManagerSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	
private:
	FText GetDisplayTokenText(FNamingTokenFilterArgs InArgs) const;
};