// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/CaptureManagerTemplateTokens.h"

#define LOCTEXT_NAMESPACE "CaptureManagerNamingTokens"


UCaptureManagerGeneralTokens::UCaptureManagerGeneralTokens()
{
	Namespace = TEXT("cpman");

	GeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::IdKey), { FString(UE::CaptureManager::GeneralTokens::IdKey), LOCTEXT("ArchiveId", "Archive Unique Id") });
	GeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::DeviceKey), { FString(UE::CaptureManager::GeneralTokens::DeviceKey), LOCTEXT("ArchiveDeviceId", "Archive Device User Id") });
	GeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::SlateKey), { FString(UE::CaptureManager::GeneralTokens::SlateKey), LOCTEXT("ArchiveSlate", "Archive Slate") });
	GeneralTokens.Emplace(FString(UE::CaptureManager::GeneralTokens::TakeKey), { FString(UE::CaptureManager::GeneralTokens::TakeKey), LOCTEXT("ArchiveTake", "Archive Take") });
}

UE::CaptureManager::FArchiveToken UCaptureManagerGeneralTokens::GetToken(const FString& InKey) const
{
	check(GeneralTokens.Contains(InKey));
	return GeneralTokens[InKey];
}

void UCaptureManagerGeneralTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens)
{
	Super::OnCreateDefaultTokens(OutTokens);

	for (const TPair<FString, UE::CaptureManager::FArchiveToken>& Token : GeneralTokens)
	{
		OutTokens.Add({
			Token.Value.Name,
			Token.Value.Description,
			FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([Name = Token.Value.Name] {
					return FText::FromString(Name);
				})
			});
	}

}

void UCaptureManagerGeneralTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);
}

void UCaptureManagerGeneralTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
}

#undef LOCTEXT_NAMESPACE
