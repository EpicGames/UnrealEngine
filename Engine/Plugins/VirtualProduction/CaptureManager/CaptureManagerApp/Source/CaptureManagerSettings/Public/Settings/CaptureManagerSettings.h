// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "CaptureManagerSettings.generated.h"

class UCaptureManagerGeneralTokens;

// Encoder token classes live in CaptureDataConverter (Core) so they are auto-discovered
// by UNamingTokensEngineSubsystem in both LiveLinkHub and UE Editor contexts.
class UCaptureManagerVideoEncoderTokens;
class UCaptureManagerAudioEncoderTokens;

UCLASS(config = Editor, defaultconfig)
class CAPTUREMANAGERSETTINGS_API UCaptureManagerSettings : public UObject
{
public:
	GENERATED_BODY()

	UCaptureManagerSettings(const FObjectInitializer& InObjectInitializer);

	/** Get the general naming tokens for Capture Manager. */
	TObjectPtr<UCaptureManagerGeneralTokens> GetGeneralNamingTokens() const;

	/** Get the video encoder naming tokens for Capture Manager. */
	TObjectPtr<UCaptureManagerVideoEncoderTokens> GetVideoEncoderNamingTokens() const;

	/** Get the audio encoder naming tokens for Capture Manager. */
	TObjectPtr<UCaptureManagerAudioEncoderTokens> GetAudioEncoderNamingTokens() const;

	/** Default location to where the converted data will be stored. It can be overriden when configuring the Ingest Job. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion")
	FDirectoryPath DefaultWorkingDirectory;

	/** Option to clean the converted data after the Ingest process. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion")
	bool bShouldCleanWorkingDirectory = true;

	/** Location where the take data downloaded from the device will be stored. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion")
	FDirectoryPath DownloadDirectory;

	/** Option to enable a third party encoder instead of the engine media readers and writers. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion", meta = (DisplayName = "Use Third Party Encoder"))
	bool bEnableThirdPartyEncoder = false;

	/** Path to the third party encoder executable. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion",
		meta = (DisplayName = "Third Party Encoder Path", EditCondition = "bEnableThirdPartyEncoder", EditConditionHides))
	FFilePath ThirdPartyEncoder;

	/** Custom video command arguments to be used for executing the third party encoder.
	 * NOTE: Leave empty to use the arguments generated from the Job settings.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Conversion",
		meta = (EditCondition = "bEnableThirdPartyEncoder", EditConditionHides))
	FString CustomVideoCommandArguments;

	/** Tokens compatible with video command properties. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion",
		meta = (EditCondition = "bEnableThirdPartyEncoder", EditConditionHides))
	FText VideoCommandTokens;

	/** Custom audio command arguments to be used for executing the third party encoder.
	 * NOTE: Leave empty to use the arguments generated from the Job settings.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Conversion",
		meta = (EditCondition = "bEnableThirdPartyEncoder", EditConditionHides))
	FString CustomAudioCommandArguments;

	/** Tokens compatible with audio command properties. */
	UPROPERTY(config, EditAnywhere, Category = "Conversion",
		meta = (EditCondition = "bEnableThirdPartyEncoder", EditConditionHides))
	FText AudioCommandTokens;

	/** General tokens */
	UPROPERTY(VisibleAnywhere, Category = "Templates")
	FText GeneralTokens;

	/** The default host to target when uploading to an Unreal client (defaults to the local host if left empty) */
	UPROPERTY(config, EditAnywhere, Category = "Upload")
	FString DefaultUploadHostName;

	/** The number of jobs to run in parallel. Requires a restart of Live Link Hub (Warning: Setting this too high could cause issues) */
	UPROPERTY(config, EditAnywhere, Category = "General", meta = (DisplayName = "Parallel Jobs", UIMin = 1, ClampMin = 1, UIMax = 8, ClampMax = 8))
	int32 NumIngestExecutors = 2;

private:

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostInitProperties() override;

	void InitializeValuesIfNotSet();

	/**
	* Naming tokens for Capture Manager, instantiated each load based on the naming tokens class.
	* This isn't serialized to the config file, and exists here for singleton-like access.
	*/
	UPROPERTY(Instanced, Transient)
	TObjectPtr<UCaptureManagerGeneralTokens> GeneralNamingTokens;

	UPROPERTY(Instanced, Transient)
	TObjectPtr<UCaptureManagerVideoEncoderTokens> VideoEncoderNamingTokens;

	UPROPERTY(Instanced, Transient)
	TObjectPtr<UCaptureManagerAudioEncoderTokens> AudioEncoderNamingTokens;

	FString LocalHostName;

};