// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "CaptureManagerTemplateTokens.generated.h"

namespace UE::CaptureManager
{
struct FArchiveToken
{
	FString Name;
	FText Description;
};

namespace GeneralTokens
{
static constexpr FStringView IdKey = TEXTVIEW("id");
static constexpr FStringView DeviceKey = TEXTVIEW("device");
static constexpr FStringView SlateKey = TEXTVIEW("slate");
static constexpr FStringView TakeKey = TEXTVIEW("take");
}

}

UCLASS(NotBlueprintable)
class CAPTUREMANAGERSETTINGS_API UCaptureManagerGeneralTokens final : public UNamingTokens
{
	GENERATED_BODY()

public:
	UCaptureManagerGeneralTokens();

	UE::CaptureManager::FArchiveToken GetToken(const FString& InKey) const;

protected:
	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& OutTokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:
	TMap<FString, UE::CaptureManager::FArchiveToken> GeneralTokens;
};

