// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CaptureManagerUnrealEndpoint : ModuleRules
{
	public CaptureManagerUnrealEndpoint(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "CMUnrealEndpoint";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerSettings",
			"CaptureManagerTakeMetadata",
			"CaptureUtils",
			"Engine",
			"LiveLinkHubCaptureMessaging",
			"MessagingCommon",
		});
	}
}
