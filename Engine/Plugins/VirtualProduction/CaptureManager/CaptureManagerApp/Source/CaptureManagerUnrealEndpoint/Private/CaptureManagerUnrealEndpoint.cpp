// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerUnrealEndpoint.h"
#include "CaptureManagerUnrealEndpointLog.h"

#include "Async/Async.h"
#include "Messenger.h"
#include "Features/ConnectStarter.h"
#include "Features/UploadStateHandler.h"

#include "LiveLinkHubExportManager.h"

#include "Tasks/Task.h"
#include "Templates/Greater.h"

#include <condition_variable>

namespace UE::CaptureManager
{

// This class just bundles together some admin for tracking and aborting take upload tasks
class FAugmentedTakeUploadTask
{
public:
	FAugmentedTakeUploadTask(TUniquePtr<FTakeUploadTask> InTakeUploadTask, TWeakPtr<FLiveLinkHubExportManager> InExportManager);

	bool Start(FLiveLinkHubExportClient::FTakeUploadParams InUploadParams);
	bool Abort();
	FGuid GetTaskID() const;

	// Return a weak ptr, just to indicate that the lifetime of the upload task itself should be governed by this object,
	// users should just Pin() temporarily to use it but not hold onto the reference forever.
	TWeakPtr<FTakeUploadTask> GetUserTask() const;

private:
	// Just to be doubly safe we don't confuse the ExportTake's INDEX_NONE return for this value, we use -2
	static constexpr int32 UnsetWorkerID = -2;

	FCriticalSection CriticalSection;
	int32 ExportManagerWorkerID = UnsetWorkerID;
	const TSharedRef<FTakeUploadTask> SharedTask;
	TWeakPtr<FLiveLinkHubExportManager> ExportManager;
	const FString LogPrefix;
};

bool operator==(const FAugmentedTakeUploadTask& InLhs, const FAugmentedTakeUploadTask& InRhs)
{
	return InLhs.GetTaskID() == InRhs.GetTaskID();
}

FAugmentedTakeUploadTask::FAugmentedTakeUploadTask(
	TUniquePtr<FTakeUploadTask> InTakeUploadTask,
	TWeakPtr<FLiveLinkHubExportManager> InExportManager
) :
	// Convert the task into a shared ref, so we can share it in callbacks
	SharedTask(MakeShareable<FTakeUploadTask>(InTakeUploadTask.Release())),
	ExportManager(MoveTemp(InExportManager)),
	LogPrefix(FString::Printf(TEXT("[Upload Task %s]"), *SharedTask->GetTaskID().ToString()))
{
}

bool FAugmentedTakeUploadTask::Start(FLiveLinkHubExportClient::FTakeUploadParams InUploadParams)
{
	FScopeLock Lock(&CriticalSection);

	if (ExportManagerWorkerID != UnsetWorkerID)
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to start upload. An upload with this ID is already in progress",
			*LogPrefix
		);
		return false;
	}

	TSharedPtr<FLiveLinkHubExportManager> SharedExportManager = ExportManager.Pin();

	if (SharedExportManager)
	{
		const int32 WorkerID = SharedExportManager->ExportTake(
			InUploadParams,
			SharedTask->GetDataDirectory(),
			SharedTask->GetTakeMetadata()
		);

		if (WorkerID == INDEX_NONE)
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Error,
				"%ls Failed to start upload. The export manager returned %d for the worker ID",
				*LogPrefix,
				WorkerID
			);
			return false;
		}

		ExportManagerWorkerID = WorkerID;
		return true;
	}
	else
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to start upload. The export manager for this task has been destroyed",
			*LogPrefix
		);
		return false;
	}
}

bool FAugmentedTakeUploadTask::Abort()
{
	TSharedPtr<FLiveLinkHubExportManager> SharedExportManager;
	int32 WorkerID = UnsetWorkerID;

	// Don't hold the lock during the ExportManager's AbortExport call, just in case we trigger some callback which
	// tries to acquire the lock again.
	{
		FScopeLock Lock(&CriticalSection);

		if (ExportManagerWorkerID == UnsetWorkerID)
		{
			// Nothing to abort - the task was likely created but no-one called Start() on it before calling Abort(),
			// so aborting does nothing.
			return true;
		}

		SharedExportManager = ExportManager.Pin();
		WorkerID = ExportManagerWorkerID;
	}

	if (SharedExportManager)
	{
		const bool bIsAborted = SharedExportManager->AbortExport(WorkerID);

		if (bIsAborted)
		{
			// Reset the upload ID just to avoid calling abort logic again with an ID that no longer exists
			FScopeLock Lock(&CriticalSection);
			ExportManagerWorkerID = UnsetWorkerID;
			return true;
		}
		else
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Error,
				"%ls Failed to abort upload. The export manager could not find a worker with this ID",
				*LogPrefix
			);
			return false;
		}
	}
	else
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to abort upload. The export manager for this take has been destroyed",
			*LogPrefix
		);
		return false;
	}
}

FGuid FAugmentedTakeUploadTask::GetTaskID() const
{
	return SharedTask->GetTaskID();
}

TWeakPtr<FTakeUploadTask> FAugmentedTakeUploadTask::GetUserTask() const
{
	return SharedTask.ToWeakPtr();
}

struct FUnrealEndpoint::FImpl : public TSharedFromThis<FUnrealEndpoint::FImpl>
{
	using FMessengerType = FMessenger<FConnectStarter, FUploadStateHandler>;

	explicit FImpl(FUnrealEndpointInfo EndpointInfo);
	~FImpl();

	void BuildMessenger();

	void StartConnection();
	void StopConnection();
	void Retire();

	bool WaitForConnectionState(const FUnrealEndpoint::EConnectionState InConnectionState, int32 InTimeoutMs) const;
	TSharedRef<FLiveLinkHubExportManager> GetOrCreateExportManager(const FGuid& InClientID, const FGuid& InCaptureSourceID);
	bool AddTakeUploadTask(TUniquePtr<FTakeUploadTask> InTakeUploadTask);
	bool TaskIDNotTracked(const FGuid& TaskID);
	void CancelTakeUploadTask(FGuid InTakeUploadTask);
	FUnrealEndpointInfo GetInfo() const;

	void OnMessengerUploadCompleted(const FGuid& InCaptureSourceID, const FGuid& InTakeUploadID, FString InMessage, int32 InCode);
	void OnMessengerUploadProgressed(const FGuid& InCaptureSourceID, const FGuid& InTakeUploadID, double InProgress);
	void OnMessengerDisconnect();
	void OnMessengerConnectResponse(const FConnectResponse& InResponse);

	mutable std::condition_variable ConditionVariable;
	mutable std::mutex Mutex;

	const FUnrealEndpointInfo EndpointInfo;
	const FString LogPrefix;

	std::atomic<bool> bConnectionIsStarted;
	bool bIsConnected;
	std::atomic<bool> bIsRetired;

	// Shared ptr so we can use the presence of this ptr as a thread safe "is ready" check also
	TSharedPtr<FMessengerType> Messenger;

	TMap<FGuid, TSharedRef<FLiveLinkHubExportManager>> ExportManagersByCaptureSourceID;
	TArray<TSharedRef<FAugmentedTakeUploadTask>> TakeUploadTasks;
};

FUnrealEndpoint::FUnrealEndpoint(FUnrealEndpointInfo EndpointInfo) :
	Impl(MakeShared<FImpl>(EndpointInfo))
{
}

FUnrealEndpoint::~FUnrealEndpoint() = default;

void FUnrealEndpoint::StartConnection()
{
	Impl->StartConnection();
}

void FUnrealEndpoint::StopConnection()
{
	Impl->StopConnection();
}

void FUnrealEndpoint::Retire()
{
	Impl->Retire();
}

bool FUnrealEndpoint::WaitForConnectionState(const FUnrealEndpoint::EConnectionState InConnectionState, int32 InTimeoutMs) const
{
	return Impl->WaitForConnectionState(InConnectionState, InTimeoutMs);
}

bool FUnrealEndpoint::AddTakeUploadTask(TUniquePtr<FTakeUploadTask> InTakeUploadTask)
{
	return Impl->AddTakeUploadTask(MoveTemp(InTakeUploadTask));
}

void FUnrealEndpoint::CancelTakeUploadTask(FGuid InTakeUploadTask)
{
	Impl->CancelTakeUploadTask(MoveTemp(InTakeUploadTask));
}

FUnrealEndpointInfo FUnrealEndpoint::GetInfo() const
{
	return Impl->GetInfo();
}

FUnrealEndpoint::FImpl::FImpl(FUnrealEndpointInfo InEndpointInfo) :
	EndpointInfo(MoveTemp(InEndpointInfo)),
	LogPrefix(FString::Printf(TEXT("[%s]"), *UnrealEndpointInfoToString(EndpointInfo))),
	bConnectionIsStarted(false),
	bIsConnected(false),
	bIsRetired(false)
{
}

FUnrealEndpoint::FImpl::~FImpl()
{
	// We do not want to acquire any mutexes or do any complex work here, so the user is required to call StopConnection
	// before the endpoint is destroyed
	checkf(!bConnectionIsStarted, TEXT("The user is expected to have called StopConnection before destruction"));
};

void FUnrealEndpoint::FImpl::BuildMessenger()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_BuildMessenger);

	TSharedPtr<FMessengerType> TheMessenger = MakeShared<FMessengerType>();

	TheMessenger->SetDisconnectHandler(FConnectStarter::FDisconnectHandler::CreateRaw(this, &FImpl::OnMessengerDisconnect));
	TheMessenger->SetUploadCallbacks(
		FUploadStateHandler::FUploadStateCallback::CreateLambda(
			[WeakThis = AsWeak()](const FGuid& InCaptureSourceID, const FGuid& InTakeUploadID, double InProgress)
			{
				using namespace UE::Tasks;

				auto Task = [WeakThis, InCaptureSourceID, InTakeUploadID, InProgress]()
					{
						TSharedPtr<FUnrealEndpoint::FImpl> This = WeakThis.Pin();

						if (This)
						{
							This->OnMessengerUploadProgressed(InCaptureSourceID, InTakeUploadID, InProgress);
						}
					};

				// Offload the task onto a background thread (i.e. never on a foreground thread, unlike AsyncTask/AnyThread).
				Launch(UE_SOURCE_LOCATION, MoveTemp(Task), ETaskPriority::BackgroundNormal);
			}
		),
		FUploadStateHandler::FUploadFinishedCallback::CreateLambda(
			[WeakThis = AsWeak()](const FGuid& InCaptureSourceID, const FGuid& InTakeUploadID, FString InMessage, const int32 InCode)
			{
				using namespace UE::Tasks;

				auto Task = [WeakThis, InCaptureSourceID, InTakeUploadID, Message = MoveTemp(InMessage), InCode]() mutable
					{
						TSharedPtr<FUnrealEndpoint::FImpl> This = WeakThis.Pin();

						if (This)
						{
							This->OnMessengerUploadCompleted(InCaptureSourceID, InTakeUploadID, MoveTemp(Message), InCode);
						}
					};

				// Offload the task onto a background thread (i.e. never on a foreground thread, unlike AsyncTask/AnyThread).
				// Basically so we don't block the message bus by doing things like directory traversal/removal.
				Launch(UE_SOURCE_LOCATION, MoveTemp(Task), ETaskPriority::BackgroundNormal);
			}
		)
	);

	// EndpointInfo is const member so no need to protect it with a mutex lock
	TheMessenger->SetAddress(EndpointInfo.MessageAddress);
	TheMessenger->Connect(FConnectStarter::FConnectHandler::CreateRaw(this, &FUnrealEndpoint::FImpl::OnMessengerConnectResponse));

	// Assign the messenger member last, so we can use it as a thread-safe indicator that the messenger is ready
	std::lock_guard<std::mutex> Lock(Mutex);
	Messenger = MoveTemp(TheMessenger);
}

void FUnrealEndpoint::FImpl::StartConnection()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_StartConnection);

	if (bIsRetired)
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Warning,
			"%ls Ignoring request to start connection, this endpoint has been retired",
			*LogPrefix
		);
		return;
	}

	if (bConnectionIsStarted)
	{
		// Nothing to do
		return;
	}

	bConnectionIsStarted = true;
	BuildMessenger();
}

void FUnrealEndpoint::FImpl::StopConnection()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_StopConnection);

	if (!bConnectionIsStarted)
	{
		// Nothing to do
		check(!bIsConnected);
		return;
	}

	// We don't hold the lock while we tear down the messenger/abort the tasks, as the callbacks for disconnection etc.
	// may try to acquire it through the public interface.
	TSharedPtr<FMessengerType> TheMessenger;
	TArray<TSharedRef<FAugmentedTakeUploadTask>> TasksToAbort;

	{
		std::lock_guard<std::mutex> Lock(Mutex);
		TheMessenger = Messenger;
		Messenger.Reset();

		TasksToAbort = MoveTemp(TakeUploadTasks);
		TakeUploadTasks.Empty();
	}

	for (TSharedRef<FAugmentedTakeUploadTask>& TaskToAbort : TasksToAbort)
	{
		const bool bIsAborted = TaskToAbort->Abort();

		if (!bIsAborted)
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Warning,
				"%ls Failed to abort take upload task %ls while stopping the connection",
				*LogPrefix,
				*TaskToAbort->GetTaskID().ToString()
			);
		}
	}

	if (TheMessenger)
	{
		TheMessenger->Disconnect();
		TheMessenger->SetDisconnectHandler(nullptr);
		TheMessenger->SetUploadCallbacks(nullptr, nullptr);

		// Make sure the messenger's teardown is complete before we set status bools/return
		TheMessenger.Reset();
	}

	check(!bIsConnected);
	bConnectionIsStarted = false;
}

void FUnrealEndpoint::FImpl::Retire()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_Retire);

	// This atomic flag is used to ensure the user doesn't try to do any meaningful work while the endpoint is being
	// stopped, this can happen if the user holds the last reference after the endpoint manager has removed its own
	// reference, the manager will call StopConnection but the user might still try to use the endpoint between then and
	// when they finally destroy the endpoint on their side. This ensures that once the manager has called retire() on the
	// endpoint, the user can't really do anything with it - the manager is responsible for the endpoint lifetime, not the
	// user.
	bIsRetired = true;

	StopConnection();
}

bool FUnrealEndpoint::FImpl::WaitForConnectionState(const FUnrealEndpoint::EConnectionState InConnectionState, const int32 TimeoutMs) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_WaitForConnectionState);

	std::unique_lock<std::mutex> Lock(Mutex);

	bool bWaitSuccess = ConditionVariable.wait_for(
		Lock,
		std::chrono::milliseconds(TimeoutMs),
		[this, InConnectionState]() -> bool
		{
			switch (InConnectionState)
			{
			case EConnectionState::Connected:
				return bIsConnected;

			case EConnectionState::Disconnected:
				return !bIsConnected;

			default:
				check(false);
				return false;
			}
		}
	);

	return bWaitSuccess;
}

TSharedRef<FLiveLinkHubExportManager> FUnrealEndpoint::FImpl::GetOrCreateExportManager(
	const FGuid& InClientID,
	const FGuid& InCaptureSourceID
)
{
	// Note: Assumes mutex lock is already held by the caller

	TSharedRef<FLiveLinkHubExportManager>* FoundExportManager = ExportManagersByCaptureSourceID.Find(InCaptureSourceID);

	if (FoundExportManager)
	{
		return *FoundExportManager;
	}

	auto OnDataUploaded = [WeakThis = AsWeak(), CaptureSourceID = InCaptureSourceID](const FGuid& InTakeUploadID, FUploadVoidResult InResult)
		{
			if (InResult.HasError())
			{
				TSharedPtr<FUnrealEndpoint::FImpl> This = WeakThis.Pin();

				if (This)
				{
					This->OnMessengerUploadCompleted(CaptureSourceID, InTakeUploadID, InResult.GetError().GetText().ToString(), InResult.GetError().GetCode());
				}
				else
				{
					UE_LOGF(
						LogCaptureManagerUnrealEndpoint,
						Error,
						"Failed to report export manager error to the endpoint, the endpoint has been destroyed: ErrorMessage=%ls, ErrorCode=%d",
						*InResult.GetError().GetText().ToString(),
						InResult.GetError().GetCode()
					);
				}
			}
		};

	return ExportManagersByCaptureSourceID.Emplace(
		InCaptureSourceID,
		MakeShared<FLiveLinkHubExportManager>(
			InClientID,
			FLiveLinkHubExportClient::FOnDataUploaded::CreateLambda(MoveTemp(OnDataUploaded))
		)
	);
};

bool FUnrealEndpoint::FImpl::AddTakeUploadTask(TUniquePtr<FTakeUploadTask> InTakeUploadTask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_AddTakeUploadTask);

	if (!ensure(InTakeUploadTask))
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to upload take, the supplied task was invalid (nullptr)",
			*LogPrefix
		);
		return false;
	}

	if (bIsRetired)
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to upload take, the endpoint has been retired",
			*LogPrefix
		);
		return false;
	}

	std::lock_guard<std::mutex> Lock(Mutex);

	if (!ensure(Messenger))
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls, Failed to upload take, the underlying messenger has not been initialized",
			*LogPrefix
		);
		return false;
	}

	FGuid ClientID;
	const bool bParseOk = FGuid::Parse(Messenger->GetOwnAddress().ToString(), ClientID);

	if (!ensure(bParseOk))
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to upload take, we failed to parse the address of the messenger",
			*LogPrefix
		);
		return false;
	}

	TSharedRef<FLiveLinkHubExportManager> ExportManager = GetOrCreateExportManager(ClientID, InTakeUploadTask->GetCaptureSourceID());

	const FGuid TaskID = InTakeUploadTask->GetTaskID();

	FLiveLinkHubExportClient::FTakeUploadParams UploadParams;
	UploadParams.CaptureSourceId = InTakeUploadTask->GetCaptureSourceID();
	UploadParams.CaptureSourceName = InTakeUploadTask->GetCaptureSourceName();
	UploadParams.TakeUploadId = TaskID;
	UploadParams.IpAddress = EndpointInfo.IPAddress;
	UploadParams.Port = EndpointInfo.ImportServicePort;

	checkf(TaskIDNotTracked(TaskID), TEXT("Task ID is already being tracked, it must be unique for each task"));

	TSharedRef<FAugmentedTakeUploadTask> TakeUploadTask = MakeShared<FAugmentedTakeUploadTask>(MoveTemp(InTakeUploadTask), ExportManager.ToWeakPtr());

	const bool bUploadIsStarted = TakeUploadTask->Start(MoveTemp(UploadParams));

	if (bUploadIsStarted)
	{
		TakeUploadTasks.Emplace(MoveTemp(TakeUploadTask));
	}

	// Failure reason should have already been logged above
	return bUploadIsStarted;
}

bool FUnrealEndpoint::FImpl::TaskIDNotTracked(const FGuid& TaskID)
{
	// Note: Assumes a mutex lock is already held by the caller

	const TSharedRef<FAugmentedTakeUploadTask>* TaskWithThisID = TakeUploadTasks.FindByPredicate(
		[&TaskID](const TSharedRef<FAugmentedTakeUploadTask>& InTask)
		{
			return InTask->GetTaskID() == TaskID;
		}
	);

	return TaskWithThisID == nullptr;
}

void FUnrealEndpoint::FImpl::CancelTakeUploadTask(const FGuid InTakeUploadTaskID)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_CancelTakeUploadTask);

	if (bIsRetired)
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to cancel task %ls. This endpoint has been retired",
			*LogPrefix,
			*InTakeUploadTaskID.ToString()
		);
		return;
	}

	std::lock_guard<std::mutex> Lock(Mutex);

	TSharedRef<FAugmentedTakeUploadTask>* TakeUploadTask = TakeUploadTasks.FindByPredicate(
		[&InTakeUploadTaskID](const TSharedRef<FAugmentedTakeUploadTask>& InTask)
		{
			return InTask->GetTaskID() == InTakeUploadTaskID;
		}
	);

	if (TakeUploadTask)
	{
		const bool bUploadIsAborted = (*TakeUploadTask)->Abort();

		if (!bUploadIsAborted)
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Error,
				"%ls Failed to abort task %ls",
				*LogPrefix,
				*(*TakeUploadTask)->GetTaskID().ToString()
			);
		}
	}

	// If the task doesn't exist, this indicates that it has already completed (and been removed) before or during this
	// cancel call
}

void FUnrealEndpoint::FImpl::OnMessengerUploadCompleted(
	const FGuid& InCaptureSourceID,
	const FGuid& InTakeUploadID,
	const FString InMessage,
	const int32 InCode
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_OnMessengerUploadCompleted);

	std::unique_lock<std::mutex> Lock(Mutex);

	const TSharedRef<FAugmentedTakeUploadTask>* FoundTask = TakeUploadTasks.FindByPredicate(
		[&InTakeUploadID](const TSharedRef<FAugmentedTakeUploadTask>& InTask)
		{
			return InTask->GetTaskID() == InTakeUploadID;
		}
	);

	if (FoundTask)
	{
		TSharedPtr<FTakeUploadTask> TakeUploadTask = (*FoundTask)->GetUserTask().Pin();

		// Don't need to track this task anymore
		const int32 Index = static_cast<int32>(FoundTask - TakeUploadTasks.GetData());
		TakeUploadTasks.RemoveAt(Index);

		Lock.unlock();

		if (TakeUploadTask)
		{
			TakeUploadTask->Complete().ExecuteIfBound(InMessage, InCode);
		}
		else
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Error,
				"%ls Failed to update completion state for task %ls (task has already been destroyed). "
					"Completion state was (Message=%ls, Code=%d)",
				*LogPrefix,
				*InTakeUploadID.ToString(),
				*InMessage,
				InCode
			);
		}
	}

	// The task may not be found and this is normal. The reason is: the OnDataUploaded handler for the export manager
	// can race against the message bus completion message to set the task state, and the first one that invokes 
	// OnMessengerUploadCompleted will win and remove the task, the one which comes after will be too late. This is 
	// how the old communication manager did things so we have preserved this behavior.
}

void FUnrealEndpoint::FImpl::OnMessengerUploadProgressed(
	const FGuid& InCaptureSourceID,
	const FGuid& InTakeUploadID,
	double InProgress
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_OnMessengerUploadProgressed);

	std::unique_lock<std::mutex> Lock(Mutex);

	const TSharedRef<FAugmentedTakeUploadTask>* TakeUploadTask = TakeUploadTasks.FindByPredicate(
		[&InTakeUploadID](const TSharedRef<FAugmentedTakeUploadTask>& InTask)
		{
			return InTask->GetTaskID() == InTakeUploadID;
		}
	);

	if (TakeUploadTask)
	{
		TSharedPtr<FTakeUploadTask> TheTakeUploadTask = (*TakeUploadTask)->GetUserTask().Pin();
		Lock.unlock();

		if (TheTakeUploadTask)
		{
			TheTakeUploadTask->Progressed().ExecuteIfBound(InProgress);
		}
		else
		{
			UE_LOGF(
				LogCaptureManagerUnrealEndpoint,
				Error,
				"%ls Failed to update progress for task %ls (task has already been destroyed). Progress was %.4f",
				*LogPrefix,
				*InTakeUploadID.ToString(),
				InProgress
			);
		}
	}
	else
	{
		UE_LOGF(
			LogCaptureManagerUnrealEndpoint,
			Error,
			"%ls Failed to update progress for task %ls (task not found). Progress was %.4f",
			*LogPrefix,
			*InTakeUploadID.ToString(),
			InProgress
		);
	}
}

void FUnrealEndpoint::FImpl::OnMessengerDisconnect()
{
	TMap<FGuid, TSharedRef<FLiveLinkHubExportManager>> LocalExportManagersByCaptureSourceID;

	{
		std::lock_guard<std::mutex> Lock(Mutex);
		LocalExportManagersByCaptureSourceID = ExportManagersByCaptureSourceID;
		ExportManagersByCaptureSourceID.Empty();

		bIsConnected = false;
	}

	LocalExportManagersByCaptureSourceID.Empty();

	ConditionVariable.notify_one();
}

void FUnrealEndpoint::FImpl::OnMessengerConnectResponse(const FConnectResponse& InResponse)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUnrealEndpoint_OnMessengerConnectResponse);

	{
		std::lock_guard<std::mutex> Lock(Mutex);
		bIsConnected = InResponse.Status == EStatus::Ok;
	}
	ConditionVariable.notify_one();

	if (!bIsConnected)
	{
		UE_LOGF(LogCaptureManagerUnrealEndpoint, Error, "%ls Failed to connect to the client", *LogPrefix);
	}
}

FUnrealEndpointInfo FUnrealEndpoint::FImpl::GetInfo() const
{
	return EndpointInfo;
}

FTakeUploadTask::FUploadProgressed& FTakeUploadTask::Progressed()
{
	return ProgressedDelegate;
}

FTakeUploadTask::FUploadComplete& FTakeUploadTask::Complete()
{
	return CompleteDelegate;
}

FTakeUploadTask::FTakeUploadTask(
	FGuid InTaskID,
	FGuid InCaptureSourceID,
	FString InCaptureSourceName,
	FString InDataDirectory,
	FTakeMetadata InTakeMetadata
) :
	TaskID(MoveTemp(InTaskID)),
	CaptureSourceID(MoveTemp(InCaptureSourceID)),
	CaptureSourceName(MoveTemp(InCaptureSourceName)),
	DataDirectory(MoveTemp(InDataDirectory)),
	TakeMetadata(MoveTemp(InTakeMetadata))
{
}

const FGuid& FTakeUploadTask::GetTaskID() const
{
	return TaskID;
}

const FGuid& FTakeUploadTask::GetCaptureSourceID() const
{
	return CaptureSourceID;
}

const FString& FTakeUploadTask::GetCaptureSourceName() const
{
	return CaptureSourceName;
}

const FString& FTakeUploadTask::GetDataDirectory() const
{
	return DataDirectory;
}

const FTakeMetadata& FTakeUploadTask::GetTakeMetadata() const
{
	return TakeMetadata;
}

FString UnrealEndpointInfoToString(const FUnrealEndpointInfo& InEndpointInfo)
{
	return FString::Printf(
		TEXT("%s:%d (%s) - %s"),
		*InEndpointInfo.IPAddress,
		InEndpointInfo.ImportServicePort,
		*InEndpointInfo.HostName,
		*InEndpointInfo.EndpointID.ToString(EGuidFormats::DigitsWithHyphens)
	);
}

}
