// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerUnrealEndpointLog.h"

DEFINE_LOG_CATEGORY(LogCaptureManagerUnrealEndpoint);
