// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerUnrealEndpointManager.h"
#include "CaptureManagerUnrealEndpointLog.h"

#include "DiscoveryRequester.h"

#include <condition_variable>

namespace UE::CaptureManager
{

struct FUnrealEndpointManager::FImpl
{
	FImpl();
	~FImpl();

	void Start();
	void Stop();

	TOptional<TWeakPtr<FUnrealEndpoint>> WaitForEndpoint(TFunction<bool(const FUnrealEndpoint&)> InPredicate, int32 InTimeoutMS);
	TOptional<TWeakPtr<FUnrealEndpoint>> FindEndpointByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate);
	TArray<TWeakPtr<FUnrealEndpoint>> GetEndpoints();
	TArray<TWeakPtr<FUnrealEndpoint>> FindEndpointsByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate);
	int32 GetNumEndpoints() const;

	TArray<TSharedRef<FUnrealEndpoint>> Endpoints;
	TUniquePtr<FDiscoveryRequester> DiscoveryRequester;
	FDelegateHandle EndpointFoundDelegateHandle;
	FDelegateHandle EndpointLostDelegateHandle;
	std::atomic<bool> bIsRunning;
	FEndpointsChanged EndpointsChangedDelegate;

	// We protect class state with a standard mutex rather than a critical section, just because we're using 
	// a condition variable and one mutex is better than two.
	mutable std::mutex Mutex;
	mutable std::condition_variable CondVar;
};

FUnrealEndpointManager::FUnrealEndpointManager() :
	Impl(MakePimpl<FImpl>())
{
}

FUnrealEndpointManager::~FUnrealEndpointManager() = default;

void FUnrealEndpointManager::Start()
{
	Impl->Start();
}

void FUnrealEndpointManager::Stop()
{
	Impl->Stop();
}

TOptional<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::WaitForEndpoint(TFunction<bool(const FUnrealEndpoint&)> InPredicate, int32 InTimeoutMS)
{
	return Impl->WaitForEndpoint(MoveTemp(InPredicate), InTimeoutMS);
}

TOptional<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FindEndpointByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate)
{
	return Impl->FindEndpointByPredicate(MoveTemp(InPredicate));
}

TArray<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FindEndpointsByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate)
{
	return Impl->FindEndpointsByPredicate(MoveTemp(InPredicate));
}

TArray<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::GetEndpoints()
{
	return Impl->GetEndpoints();
}

int32 FUnrealEndpointManager::GetNumEndpoints() const
{
	return Impl->GetNumEndpoints();
}

FUnrealEndpointManager::FEndpointsChanged& FUnrealEndpointManager::EndpointsChanged()
{
	return Impl->EndpointsChangedDelegate;
}

FUnrealEndpointManager::FImpl::FImpl() :
	bIsRunning(false)
{
}

FUnrealEndpointManager::FImpl::~FImpl()
{
	// Stop should have been called before destruction
	check(!bIsRunning);
}

void FUnrealEndpointManager::FImpl::Start()
{
	if (bIsRunning)
	{
		return;
	}

	TUniquePtr<FDiscoveryRequester> NewRequester = FDiscoveryRequester::Create();

	std::lock_guard<std::mutex> LockGuard(Mutex);

	DiscoveryRequester = MoveTemp(NewRequester);

	if (!DiscoveryRequester)
	{
		UE_LOGF(LogCaptureManagerUnrealEndpoint, Warning, "Endpoint manager failed to start (Discovery is disabled)");
		return;
	}

	bIsRunning = true;

	EndpointFoundDelegateHandle = DiscoveryRequester->ClientFound().AddLambda(
		[this](const FDiscoveredClient& InDiscoveredClient)
		{
			FUnrealEndpointInfo EndpointInfo =
			{
				.EndpointID = InDiscoveredClient.GetClientID(),
				.MessageAddress = InDiscoveredClient.GetMessageAddress(),
				.IPAddress = InDiscoveredClient.GetIPAddress(),
				.HostName = InDiscoveredClient.GetHostName(),
				.ImportServicePort = InDiscoveredClient.GetExportPort()
			};

			TSharedRef<FUnrealEndpoint> Endpoint = MakeShared<FUnrealEndpoint>(EndpointInfo);

			{
				std::lock_guard<std::mutex> LockGuard(Mutex);
				Endpoints.Emplace(MoveTemp(Endpoint));
			}
			CondVar.notify_one();

			EndpointsChangedDelegate.Broadcast();
		}
	);

	EndpointLostDelegateHandle = DiscoveryRequester->ClientLost().AddLambda(
		[this](const FGuid& InEndpointId)
		{
			std::unique_lock<std::mutex> LockGuard(Mutex);

			const TSharedRef<FUnrealEndpoint>* FoundEndpoint = Endpoints.FindByPredicate(
				[&InEndpointId](const TSharedRef<FUnrealEndpoint>& InEndpoint)
				{
					return InEndpoint->GetInfo().EndpointID == InEndpointId;
				}
			);

			if (FoundEndpoint)
			{
				const int32 IndexToRemove = static_cast<int32>(FoundEndpoint - Endpoints.GetData());

				// Retire the endpoint and prevent others from using/restarting it. The manager is no longer tracking 
				// this endpoint so we don't want the user to hold onto it or to start it back up.
				(*FoundEndpoint)->Retire();
				Endpoints.RemoveAt(IndexToRemove);
			}

			LockGuard.unlock();
			EndpointsChangedDelegate.Broadcast();

		}
	);

	DiscoveryRequester->Start();
}

void FUnrealEndpointManager::FImpl::Stop()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	if (DiscoveryRequester)
	{
		DiscoveryRequester->ClientFound().Remove(EndpointFoundDelegateHandle);
		DiscoveryRequester->ClientLost().Remove(EndpointLostDelegateHandle);
		DiscoveryRequester.Reset();
	}

	for (const TSharedRef<FUnrealEndpoint>& Endpoint : Endpoints)
	{
		Endpoint->StopConnection();
	}

	Endpoints.Empty();
	bIsRunning = false;
}

TOptional<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FImpl::WaitForEndpoint(TFunction<bool(const FUnrealEndpoint&)> InPredicate, const int32 InTimeoutMS)
{
	TOptional<TWeakPtr<FUnrealEndpoint>> Endpoint;

	std::unique_lock<std::mutex> Lock(Mutex);

	[[maybe_unused]] bool bWaitSuccess = CondVar.wait_for(
		Lock,
		std::chrono::milliseconds(InTimeoutMS),
		[this, &InPredicate, &Endpoint]() -> bool
		{
			TSharedRef<FUnrealEndpoint>* FoundEndpoint = Endpoints.FindByPredicate(
				[&InPredicate](const TSharedRef<FUnrealEndpoint>& InEndpoint)
				{
					// We convert the internal shared ref into a simple reference, just to make the caller's life a bit easier
					return InPredicate(*InEndpoint);
				}
			);

			if (FoundEndpoint)
			{
				Endpoint = FoundEndpoint->ToWeakPtr();
			}

			return Endpoint.IsSet();
		}
	);

	return Endpoint;
}

TOptional<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FImpl::FindEndpointByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate)
{
	TOptional<TWeakPtr<FUnrealEndpoint>> Endpoint;

	std::lock_guard<std::mutex> LockGuard(Mutex);

	TSharedRef<FUnrealEndpoint>* FoundEndpoint = Endpoints.FindByPredicate(
		[&InPredicate](const TSharedRef<FUnrealEndpoint>& InEndpoint)
		{
			return InPredicate(*InEndpoint);
		}
	);

	if (FoundEndpoint)
	{
		Endpoint = FoundEndpoint->ToWeakPtr();
	}

	return Endpoint;
}

TArray<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FImpl::GetEndpoints()
{
	TArray<TWeakPtr<FUnrealEndpoint>> WeakEndpoints;

	std::lock_guard<std::mutex> LockGuard(Mutex);
	WeakEndpoints.Reserve(Endpoints.Num());

	for (const TSharedRef<FUnrealEndpoint>& Endpoint : Endpoints)
	{
		WeakEndpoints.Emplace(Endpoint.ToWeakPtr());
	}

	return WeakEndpoints;
}

TArray<TWeakPtr<FUnrealEndpoint>> FUnrealEndpointManager::FImpl::FindEndpointsByPredicate(TFunction<bool(const FUnrealEndpoint&)> InPredicate)
{
	TArray<TWeakPtr<FUnrealEndpoint>> FilteredEndpoints;

	std::lock_guard<std::mutex> LockGuard(Mutex);

	for (const TSharedRef<FUnrealEndpoint>& Endpoint : Endpoints)
	{
		if (InPredicate(*Endpoint))
		{
			FilteredEndpoints.Emplace(Endpoint.ToWeakPtr());
		}
	}

	return FilteredEndpoints;
}

int32 FUnrealEndpointManager::FImpl::GetNumEndpoints() const
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	return Endpoints.Num();
}

}
