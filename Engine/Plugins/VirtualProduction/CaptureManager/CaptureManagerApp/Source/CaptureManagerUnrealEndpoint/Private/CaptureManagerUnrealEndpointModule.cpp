// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureManagerUnrealEndpointModule.h"

void FCaptureManagerUnrealEndpointModule::StartupModule()
{
	EndpointManager = MakeShared<UE::CaptureManager::FUnrealEndpointManager>();
}

void FCaptureManagerUnrealEndpointModule::ShutdownModule()
{
	EndpointManager.Reset();
}

TSharedRef<UE::CaptureManager::FUnrealEndpointManager> FCaptureManagerUnrealEndpointModule::GetEndpointManager() const
{
	return EndpointManager.ToSharedRef();
}

TOptional<TSharedRef<UE::CaptureManager::FUnrealEndpointManager>> FCaptureManagerUnrealEndpointModule::GetEndpointManagerIfValid() const
{
	if (EndpointManager.IsValid())
	{
		return EndpointManager.ToSharedRef();
	}

	return {};
}

IMPLEMENT_MODULE(FCaptureManagerUnrealEndpointModule, CaptureManagerUnrealEndpoint);