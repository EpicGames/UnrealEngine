// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscoveryRequester.h"

#include "LiveLinkHubCaptureMessages.h"

#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"

#include "Async/CaptureTimerManager.h"

#include "CaptureUtilsModule.h"
#include "Network/NetworkMisc.h"

DEFINE_LOG_CATEGORY(LogLiveLinkHubDiscovery);

namespace UE::CaptureManager
{

FDiscoveredClient::FDiscoveredClient(FGuid InClientID,
	FString InHostName,
	FString InIPAddress,
	uint16 InExportPort,
	FMessageAddress InMessageAddress)
	: ClientID(MoveTemp(InClientID))
	, HostName(MoveTemp(InHostName))
	, IPAddress(MoveTemp(InIPAddress))
	, ExportPort(InExportPort)
	, MessageAddress(MoveTemp(InMessageAddress))
{
}

FDiscoveredClient::FDiscoveredClient(const FDiscoveredClient& InOther)
	: ClientID(InOther.ClientID)
	, HostName(InOther.HostName)
	, IPAddress(InOther.IPAddress)
	, ExportPort(InOther.ExportPort)
	, MessageAddress(InOther.MessageAddress)
	, LastDiscoveryResponse(InOther.LastDiscoveryResponse.load())
{
}

FDiscoveredClient::FDiscoveredClient(FDiscoveredClient&& InOther)
	: ClientID(MoveTemp(InOther.ClientID))
	, HostName(MoveTemp(InOther.HostName))
	, IPAddress(MoveTemp(InOther.IPAddress))
	, ExportPort(InOther.ExportPort)
	, MessageAddress(MoveTemp(InOther.MessageAddress))
	, LastDiscoveryResponse(InOther.LastDiscoveryResponse.load())
{
}

bool FDiscoveredClient::IsActive() const
{
	const double CurrentLastDiscoveryResponse = LastDiscoveryResponse.load();
	const double CurrentTime = FPlatformTime::Seconds();

	return CurrentTime - CurrentLastDiscoveryResponse < InactiveTimeout;
}

FGuid FDiscoveredClient::GetClientID() const
{
	return ClientID;
}

FString FDiscoveredClient::GetHostName() const
{
	return HostName;
}

FString FDiscoveredClient::GetIPAddress() const
{
	return IPAddress;
}

uint16 FDiscoveredClient::GetExportPort() const
{
	return ExportPort;
}

FMessageAddress FDiscoveredClient::GetMessageAddress() const
{
	return MessageAddress;
}

void FDiscoveredClient::SetLastDiscoveryResponse(double InLastDiscoveryResponse)
{
	LastDiscoveryResponse.store(InLastDiscoveryResponse);
}

bool operator==(const FDiscoveredClient& InLeft, const FDiscoveredClient& InRight)
{
	return InLeft.MessageAddress == InRight.MessageAddress;
}

struct FDiscoveryRequester::FImpl
{
	static TSharedRef<FCaptureTimerManager> GetTimerManager();

	static TUniquePtr<FImpl> Create(FString InLocalHostName);
	~FImpl();

	void OnTick();
	void HandleDiscoveryResponse(
		const FDiscoveryResponse& InResponse,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
	);

	void StartDiscoveryRequester();
	TArray<FDiscoveredClient> GetDiscoveredClients() const;

	FString HostName;

	static constexpr float DiscoveryRequestPeriod = 15.0f;

	FCaptureTimerManager::FTimerHandle TickerHandle;
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	FClientFound ClientFoundDelegate;
	FClientLost ClientLostDelegate;

private:
	explicit FImpl(FString InLocalHostName);

	void RemoveStaleClients();
	mutable FCriticalSection Mutex;
	TArray<FDiscoveredClient> KnownClients;

	TSharedRef<FCaptureTimerManager> TimerManager;
};

TSharedRef<FCaptureTimerManager> FDiscoveryRequester::FImpl::GetTimerManager()
{
	FCaptureUtilsModule& CaptureUtils =
		FModuleManager::LoadModuleChecked<FCaptureUtilsModule>("CaptureUtils");
	return CaptureUtils.GetTimerManager();
}

TUniquePtr<FDiscoveryRequester::FImpl> FDiscoveryRequester::FImpl::Create(FString InLocalHostName)
{
	TUniquePtr<FImpl> Impl = TUniquePtr<FImpl>(new FImpl(MoveTemp(InLocalHostName)));

	// Message endpoint must be created on the game thread
	check(IsInGameThread());

	TSharedPtr<FMessageEndpoint> MessageEndpoint = FMessageEndpoint::Builder("DiscoveryRequester")
		.Handling<FDiscoveryResponse>(Impl.Get(), &FImpl::HandleDiscoveryResponse)
		.ReceivingOnAnyThread()
		;

	if (!MessageEndpoint)
	{
		UE_LOGF(
			LogLiveLinkHubDiscovery,
			Warning,
			"Failed to create message endpoint, discovery will be disabled"
		);

		return nullptr;
	}

	Impl->MessageEndpoint = MoveTemp(MessageEndpoint);
	return Impl;
}

FDiscoveryRequester::FImpl::FImpl(FString InLocalHostName) :
	HostName(MoveTemp(InLocalHostName)),
	TimerManager(GetTimerManager())
{
}

FDiscoveryRequester::FImpl::~FImpl()
{
	FMessageEndpoint::SafeRelease(MessageEndpoint);

	TimerManager->RemoveTimer(TickerHandle);
}

TArray<FDiscoveredClient> FDiscoveryRequester::FImpl::GetDiscoveredClients() const
{
	FScopeLock Lock(&Mutex);
	return KnownClients;
}

void FDiscoveryRequester::FImpl::RemoveStaleClients()
{
	// We store the inactive clients just so we aren't iterating and removing from the array at the same time
	TArray<FDiscoveredClient> InactiveClients;

	// Store the client IDs so we can broadcast them all at the end (outside the scope lock)
	TArray<FGuid> LostClientIDs;

	FScopeLock Lock(&Mutex);

	for (const FDiscoveredClient& KnownClient : KnownClients)
	{
		if (!KnownClient.IsActive())
		{
			InactiveClients.Emplace(KnownClient);
		}
	}

	for (const FDiscoveredClient& InActiveClient : InactiveClients)
	{
		FGuid ClientID = InActiveClient.GetClientID();
		LostClientIDs.Emplace(MoveTemp(ClientID));
		KnownClients.Remove(InActiveClient);
	}

	Lock.Unlock();

	// Broadcast now we don't need the lock anymore
	for (const FGuid& LostClientId : LostClientIDs)
	{
		ClientLostDelegate.Broadcast(LostClientId);
	}
}

TArray<FDiscoveredClient> FDiscoveryRequester::GetDiscoveredClients() const
{
	return Impl->GetDiscoveredClients();
}

FDiscoveryRequester::FClientFound& FDiscoveryRequester::ClientFound()
{
	return Impl->ClientFoundDelegate;
}

FDiscoveryRequester::FClientLost& FDiscoveryRequester::ClientLost()
{
	return Impl->ClientLostDelegate;
}

void FDiscoveryRequester::FImpl::OnTick()
{
	check(!IsInGameThread());

	FDiscoveryRequest* Message = FMessageEndpoint::MakeMessage<FDiscoveryRequest>();
	Message->HostName = HostName;

	check(MessageEndpoint);
	MessageEndpoint->Publish(Message);

	RemoveStaleClients();
}

void FDiscoveryRequester::FImpl::HandleDiscoveryResponse(
	const FDiscoveryResponse& InResponse,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	check(!IsInGameThread());

	UE_LOGF(LogLiveLinkHubDiscovery, Verbose, "Getting discovery response from client: %ls, (%ls)", *InContext->GetSender().ToString(), *InResponse.HostName);

	FScopeLock Lock(&Mutex);

	FDiscoveredClient* KnownClient =
		KnownClients.FindByPredicate([MessageAddress = InContext->GetSender()](const FDiscoveredClient& InDiscoveredClient)
			{
				return InDiscoveredClient.GetMessageAddress() == MessageAddress;
			});

	if (KnownClient)
	{
		KnownClient->SetLastDiscoveryResponse(FPlatformTime::Seconds());
		return;
	}

	// We bind the client ID to the message ID. It's just a convenient GUID.
	FGuid ClientID;
	bool bParseOK = FGuid::Parse(InContext->GetSender().ToString(), ClientID);

	check(bParseOK);

	if (bParseOK)
	{
		FDiscoveredClient Client(MoveTemp(ClientID), InResponse.HostName, InResponse.IPAddress, InResponse.ExportPort, InContext->GetSender());
		Client.SetLastDiscoveryResponse(FPlatformTime::Seconds());

		KnownClients.Emplace(Client);
		Lock.Unlock();

		UE_LOGF(LogLiveLinkHubDiscovery, Display, "New client discovered %ls with IP address %ls and endpoint ID: %ls", *Client.GetHostName(), *Client.GetIPAddress(), *Client.GetClientID().ToString());
		ClientFoundDelegate.Broadcast(Client);
	}
}

void FDiscoveryRequester::FImpl::StartDiscoveryRequester()
{
	TickerHandle = TimerManager->AddTimer(FTimerDelegate::CreateRaw(this, &FImpl::OnTick), DiscoveryRequestPeriod, true);
}

TUniquePtr<FDiscoveryRequester> FDiscoveryRequester::Create()
{
	TOptional<FString> LocalHostName = GetLocalHostName();

	if (!LocalHostName.IsSet())
	{
		UE_LOGF(
			LogLiveLinkHubDiscovery,
			Warning,
			"Failed to determine local host name, discovery will be disabled"
		);
		return nullptr;
	}

	if (LocalHostName.GetValue().IsEmpty())
	{
		UE_LOGF(
			LogLiveLinkHubDiscovery,
			Warning,
			"The local host name was invalid (empty), discovery will be disabled"
		);
		return nullptr;
	}

	// We construct the Impl in this way to avoid constructing the requester into an invalid state if the message endpoint build fails.
	TUniquePtr<FImpl> Impl = FImpl::Create(MoveTemp(*LocalHostName));

	if (!Impl)
	{
		return nullptr;
	}

	return TUniquePtr<FDiscoveryRequester>(new FDiscoveryRequester(MoveTemp(Impl)));
}

FDiscoveryRequester::FDiscoveryRequester(TUniquePtr<FImpl> InImpl)
	: Impl(MoveTemp(InImpl))
{
}

FDiscoveryRequester::~FDiscoveryRequester() = default;

void FDiscoveryRequester::Start()
{
	Impl->StartDiscoveryRequester();
}

} // namespace UE::CaptureManager
