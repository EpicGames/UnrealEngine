// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessageEndpoint.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkHubDiscovery, Log, All);

namespace UE::CaptureManager
{

class FDiscoveredClient
{
public:

	FDiscoveredClient(FGuid InClientID,
					  FString InHostName, 
					  FString InIPAddress, 
					  uint16 InPort, 
					  FMessageAddress InMessageAddress);

	FDiscoveredClient(const FDiscoveredClient& InOther);
	FDiscoveredClient(FDiscoveredClient&& InOther);

	bool IsActive() const;

	FGuid GetClientID() const;
	FString GetHostName() const;
	FString GetIPAddress() const;
	uint16 GetExportPort() const;
	FMessageAddress GetMessageAddress() const;

	void SetLastDiscoveryResponse(double InLastDiscoveryResponse);

	friend bool operator==(const FDiscoveredClient& InLeft, const FDiscoveredClient& InRight);

private:

	static constexpr double InactiveTimeout = 20.0f;

	FGuid ClientID;
	FString HostName;
	FString IPAddress;
	uint16 ExportPort;

	FMessageAddress MessageAddress;

	std::atomic<double> LastDiscoveryResponse = 0.0f;
};

class FDiscoveryRequester
{
public:
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FClientFound, const FDiscoveredClient& InClient);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FClientLost, const FGuid& InClientId);

	static TUniquePtr<FDiscoveryRequester> Create();
	~FDiscoveryRequester();

	void Start();
	TArray<FDiscoveredClient> GetDiscoveredClients() const;

	FClientFound& ClientFound();
	FClientLost& ClientLost();

private:
	struct FImpl;

	explicit FDiscoveryRequester(TUniquePtr<FImpl> InImpl);

	TUniquePtr<FImpl> Impl;
};

} // namespace UE::CaptureManager
