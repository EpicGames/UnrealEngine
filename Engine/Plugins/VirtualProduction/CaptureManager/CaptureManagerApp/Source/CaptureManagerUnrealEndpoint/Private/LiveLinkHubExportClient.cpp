// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubExportClient.h"

#include "Network/TcpClient.h"

#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Containers/StaticArray.h"

#include "Async/Async.h"

#define LOCTEXT_NAMESPACE "LiveLinkHubExportClient"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkHubExportClient, Log, All);

FLiveLinkHubExportClient::FLiveLinkHubExportClient(FGuid InClientId, FOnDataUploaded InOnDataUploaded)
	: ClientId(InClientId)
	, OnDataUploaded(MoveTemp(InOnDataUploaded))
	, TcpClient(MakeUnique<UE::CaptureManager::FTcpClient>())
	, UploadQueueRunner(UE::CaptureManager::TQueueRunner<TUniquePtr<FTransferContext>>::FOnProcess::CreateRaw(this, &FLiveLinkHubExportClient::OnUploadTake))
	, FileManager(IFileManager::Get())
{
}

FLiveLinkHubExportClient::~FLiveLinkHubExportClient()
{
	UploadQueueRunner.Empty();

	StopRequester.RequestStop();
}

void FLiveLinkHubExportClient::AddTakeForUpload(const FTakeUploadParams& InTakeUploadParams,
												const FString& InTakeDirectory,
												const FTakeMetadata& InTakeMetadata)
{
	using namespace UE::CaptureManager;

	uint64 TotalSizeBytes = GetTotalSizeBytes(InTakeDirectory);

	UploadQueueRunner.Add(MakeUnique<FTransferContext>(InTakeUploadParams, InTakeDirectory, InTakeMetadata, TotalSizeBytes));

	++TaskCounter;
}

void FLiveLinkHubExportClient::AbortCurrentTakeUpload()
{
	StopRequester.RequestStop();
}

int32 FLiveLinkHubExportClient::GetTaskCount() const
{
	return TaskCounter;
}

bool FLiveLinkHubExportClient::HasTasks() const
{
	return TaskCounter == 0;
}

FUploadVoidResult FLiveLinkHubExportClient::SendTakeHeader(const FTakeUploadParams& InTakeUploadParams, const FTakeMetadata& InTake, const uint64 InTakeTotalLength)
{
	using namespace UE::CaptureManager;

	FTcpClientWriter Writer(*TcpClient);

	FUploadVoidResult Result = FUploadDataMessage::SerializeHeader({ ClientId, InTakeUploadParams.CaptureSourceId, InTakeUploadParams.TakeUploadId, InTakeUploadParams.CaptureSourceName, InTake.Slate, InTake.TakeNumber, InTakeTotalLength }, Writer);

	if (Result.HasError())
	{
		return Result;
	}

	return MakeValue();
}

FUploadVoidResult FLiveLinkHubExportClient::SendFile(const FString& InFileName, const FString& InFilePath, const UE::CaptureManager::FStopToken& InStopToken)
{
	FUploadVoidResult FileHeaderResult = SendFileHeader(InFileName, InFilePath);
	if (FileHeaderResult.HasError())
	{
		return FileHeaderResult;
	}

	FUploadVoidResult FileDataResult = SendFileData(InFilePath, InStopToken);
	if (FileDataResult.HasError())
	{
		return FileDataResult;
	}

	return MakeValue();
}

FUploadVoidResult FLiveLinkHubExportClient::SendFileHeader(const FString& InFileName, const FString& InFilePath)
{
	using namespace UE::CaptureManager;

	FTcpClientWriter Writer(*TcpClient);

	const uint64 FileSize = FileManager.FileSize(*InFilePath);

	FUploadVoidResult Result = FUploadDataMessage::SerializeFileHeader({ InFileName, FileSize }, Writer);

	return Result;
}

FUploadVoidResult FLiveLinkHubExportClient::SendFileData(const FString& InFilePath, const UE::CaptureManager::FStopToken& InStopToken)
{
	using namespace UE::CaptureManager;

	FTcpClientWriter Writer(*TcpClient);

	TUniquePtr<FArchive> Reader(FileManager.CreateFileReader(*InFilePath));

	if (!Reader.IsValid())
	{
		FText Message = 
			FText::Format(LOCTEXT("SendFileData_ReaderError", "Failed to read the file requested for sending: {0}"), FText::FromString(InFilePath));
		return MakeError(MoveTemp(Message));
	}

	ON_SCOPE_EXIT
	{
		Reader->Close();
	};

	FMD5 MD5Generator;

	constexpr int64 NumChunkBytes = 64 * 1024;
	int64 NumRemainingBytes = Reader->TotalSize();

	while (NumRemainingBytes != 0)
	{
		if (InStopToken.IsStopRequested())
		{
			static const int32 AbortedByUser = -20;
			return MakeError(LOCTEXT("SendFileData_StopRequested", "Sending file is canceled by the user"), AbortedByUser);
		}

		TArray<uint8> Data;

		uint64 NumBytesToWrite = FMath::Min(NumRemainingBytes, NumChunkBytes);

		Data.SetNum(NumBytesToWrite);

		Reader->Serialize(Data.GetData(), NumBytesToWrite);

		MD5Generator.Update(Data.GetData(), Data.Num());

		FUploadVoidResult Result = FUploadDataMessage::SerializeData(Data, Writer);

		if (Result.HasError())
		{
			FUploadError Error = Result.StealError();
			return MakeError(MoveTemp(Error));
		}

		NumRemainingBytes -= Data.Num();
	}

	TStaticArray<uint8, FUploadDataMessage::HashSize> Hash;
	MD5Generator.Final(Hash.GetData());

	return FUploadDataMessage::SerializeHash(Hash, Writer);
}

uint64 FLiveLinkHubExportClient::GetTotalSizeBytes(const FString& InTakeStorage) const
{
	uint64 TotalSizeBytes = 0;

	FileManager.IterateDirectoryRecursively(*InTakeStorage, [this, &TotalSizeBytes](const TCHAR* PathName, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			uint64 Size = FileManager.FileSize(PathName);
			TotalSizeBytes += Size;
		}

		return true;
	});

	return TotalSizeBytes;
}

void FLiveLinkHubExportClient::OnUploadTake(TUniquePtr<FTransferContext> InTransferContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHubExportClient_OnUploadTake);

	if (!InTransferContext)
	{
		return;
	}

	FTransferContext& Context = *InTransferContext;
	const FTakeUploadParams& TakeUploadParams = Context.TakeUploadParams;

	ON_SCOPE_EXIT
	{
		--TaskCounter;
	};

	// Currently this will stop all tasks currently listening to the stop token, we may want to revisit this in future
	// to allow selectively stopping tasks.
	const bool bIsStopRequested = StopRequester.IsStopRequested();

	if (bIsStopRequested)
	{
		OnDataUploaded.ExecuteIfBound(TakeUploadParams.TakeUploadId, MakeError(LOCTEXT("UploadTcpClient_TakeHeaderAborted", "Aborted by the user")));
		Disconnect();
		return;
	}

	UE::CaptureManager::FStopToken Token = StopRequester.CreateToken();

	FPaths::NormalizeDirectoryName(Context.TakeStorage);

	const FTakeMetadata& TakeMetadata = Context.TakeMetadata;

	FUploadVoidResult TakeHeaderResult = MakeValue();
	constexpr int32 NumberOfRetries = 3;
	for (int32 Retry = 0; Retry < NumberOfRetries; ++Retry)
	{
		bool bConnected = RestartConnection(TakeUploadParams.IpAddress, TakeUploadParams.Port);

		if (bConnected)
		{
			TakeHeaderResult = SendTakeHeader(TakeUploadParams, Context.TakeMetadata, Context.TotalSizeBytes);

			if (!TakeHeaderResult.HasError())
			{
				if (Token.IsStopRequested())
				{
					TakeHeaderResult = MakeError(LOCTEXT("UploadTcpClient_TakeHeaderAborted", "Aborted by the user"));
				}

				break;
			}
		}
		else
		{
			TakeHeaderResult = MakeError(LOCTEXT("UploadTcpClient_TakeHeaderFailedToConnect", "Failed to connect to the server"));
		}
	}

	if (TakeHeaderResult.HasError())
	{
		FUploadError UploadError = TakeHeaderResult.StealError();

		FString TakeName = TakeMetadata.Slate + TEXT("_") + FString::FromInt(TakeMetadata.TakeNumber);

		UE_LOGF(LogLiveLinkHubExportClient, Error, "Serialize header for take: %ls %ls %d", *TakeName, *UploadError.GetText().ToString(), UploadError.GetCode());

		AbortCurrentTakeUpload();
		OnDataUploaded.ExecuteIfBound(TakeUploadParams.TakeUploadId, MakeError(MoveTemp(UploadError)));

		Disconnect();

		return;
	}

	TArray<FString> FilesToSend;

	FUploadVoidResult FileUploadResult = MakeValue();
	FileManager.IterateDirectoryRecursively(*Context.TakeStorage, [this, &Token, &FilesToSend](const TCHAR* PathName, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			FilesToSend.Add(PathName);
		}

		// Returns false to exit the loop if user canceled or there is an error while sending a file
		return !(Token.IsStopRequested());
	});

	for (const FString& FilePath : FilesToSend)
	{
		if (Token.IsStopRequested())
		{
			FileUploadResult = MakeError(LOCTEXT("UploadTcpClient_Aborted", "Aborted by the user"));
			break;
		}

		FString FileName(FilePath);
		FileName = FileName.Mid(Context.TakeStorage.Len() + 1); // Removing trailing slash

		FileUploadResult = SendFile(FileName, FilePath, Token);
	}

	OnDataUploaded.ExecuteIfBound(TakeUploadParams.TakeUploadId, MoveTemp(FileUploadResult));

	Disconnect();
}

bool FLiveLinkHubExportClient::RestartConnection(const FString& InIpAddress, uint16 InPort)
{
	using namespace UE::CaptureManager;

	TcpClient->Stop();

	TcpClient->Init();

	FString IpAddress = FString::Format(TEXT("{0}:{1}"), { InIpAddress, FString::FromInt(InPort) });
	TProtocolResult<void> ConnectError = TcpClient->Start(IpAddress);

	return ConnectError.HasValue();
}

bool FLiveLinkHubExportClient::Disconnect()
{
	return TcpClient->Stop().HasValue();
}

#undef LOCTEXT_NAMESPACE