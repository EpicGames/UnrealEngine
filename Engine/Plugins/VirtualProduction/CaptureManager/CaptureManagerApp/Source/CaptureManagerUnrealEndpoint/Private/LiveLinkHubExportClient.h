// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerTakeMetadata.h"
#include "Async/StopToken.h"

#include "Templates/SharedPointer.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/FileManager.h"

#include "UploadDataMessage.h"

#include "Async/QueueRunner.h"

namespace UE::CaptureManager
{
class FTcpClient;
}

class FLiveLinkHubExportClient
	: public TSharedFromThis<FLiveLinkHubExportClient>
{
public:

	DECLARE_DELEGATE_TwoParams(FOnDataUploaded, const FGuid& InTakeUploadId, FUploadVoidResult InResult);

	struct FTakeUploadParams
	{
		FGuid CaptureSourceId;
		FString CaptureSourceName;
		FGuid TakeUploadId;
		FString IpAddress;
		uint16 Port = 0;
	};

	FLiveLinkHubExportClient(FGuid InClientId, FOnDataUploaded InOnDataUploaded);
	~FLiveLinkHubExportClient();

	void AddTakeForUpload(const FTakeUploadParams& TakeUploadParams,
						  const FString& InTakeDirectory,
						  const FTakeMetadata& InTakeMetadata);

	void AbortCurrentTakeUpload();

	int32 GetTaskCount() const;
	bool HasTasks() const;

private:

	struct FTransferContext
	{
		FTransferContext(const FTakeUploadParams& InTakeUploadParams,
						 FString InTakeStorage,
						 FTakeMetadata InTakeMetadata,
						 uint64 InTotalSizeBytes)
			: TakeUploadParams(InTakeUploadParams)
			, TakeStorage(MoveTemp(InTakeStorage))
			, TakeMetadata(MoveTemp(InTakeMetadata))
			, TotalSizeBytes(InTotalSizeBytes)
		{
		}

		FTakeUploadParams TakeUploadParams;

		FString TakeStorage;
		FTakeMetadata TakeMetadata;
		uint64 TotalSizeBytes;
	};

	FUploadVoidResult SendTakeHeader(const FTakeUploadParams& InTakeUploadParams, const FTakeMetadata& InTake, const uint64 InTotalLength);
	FUploadVoidResult SendFile(const FString& InFileName, const FString& InFilePath, const UE::CaptureManager::FStopToken& InStopToken);
	FUploadVoidResult SendFileHeader(const FString& InFileName, const FString& InFilePath);
	FUploadVoidResult SendFileData(const FString& InFilePath, const UE::CaptureManager::FStopToken& InStopToken);

	uint64 GetTotalSizeBytes(const FString& InTakeStorage) const;

	void OnUploadTake(TUniquePtr<FTransferContext> InTransferContext);
	bool RestartConnection(const FString& InIpAddress, uint16 InPort);
	bool Disconnect();

	FGuid ClientId;
	FOnDataUploaded OnDataUploaded;

	std::atomic<int32> TaskCounter = 0;
	UE::CaptureManager::FStopRequester StopRequester;
	TUniquePtr<UE::CaptureManager::FTcpClient> TcpClient;
	UE::CaptureManager::TQueueRunner<TUniquePtr<FTransferContext>> UploadQueueRunner;

	IFileManager& FileManager;
};
