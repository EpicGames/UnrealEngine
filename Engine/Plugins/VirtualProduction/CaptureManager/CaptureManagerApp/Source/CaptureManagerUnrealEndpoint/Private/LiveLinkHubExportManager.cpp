// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubExportManager.h"

FLiveLinkHubExportManager::FLiveLinkHubExportManager(FGuid InClientId,
													 FLiveLinkHubExportClient::FOnDataUploaded InDataUploaded, 
													 uint64 InDefaultNumberOfWorkers)
	: ClientId(InClientId)
{
	for (int32 WorkerIndex = 0; WorkerIndex < InDefaultNumberOfWorkers; ++WorkerIndex)
	{
		TSharedPtr<FLiveLinkHubExportClient> Worker = MakeShared<FLiveLinkHubExportClient>(ClientId, InDataUploaded);
		Workers.Add(MoveTemp(Worker));
	}
}

FLiveLinkHubExportManager::~FLiveLinkHubExportManager()
{
	FScopeLock Lock(&Mutex);

	for (TSharedPtr<FLiveLinkHubExportClient>& Worker : Workers)
	{
		Worker->AbortCurrentTakeUpload();
	}

	Workers.Empty();
}

int32 FLiveLinkHubExportManager::ExportTake(const FLiveLinkHubExportClient::FTakeUploadParams& InTakeUploadParams,
											const FString& InTakeDirectory,
											const FTakeMetadata& InTakeMetadata)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHubExportManager_ExportTake);

	FScopeLock Lock(&Mutex);

	if (Workers.IsEmpty())
	{
		return INDEX_NONE;
	}

	int32 PreferredWorkerIndex = GetPreferredWorkerIndex();
	Workers[PreferredWorkerIndex]->AddTakeForUpload(InTakeUploadParams, InTakeDirectory, InTakeMetadata);

	return PreferredWorkerIndex;
}

bool FLiveLinkHubExportManager::AbortExport(int32 InWorkerIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLiveLinkHubExportManager_AbortExport);

	FScopeLock Lock(&Mutex);

	if (Workers.IsValidIndex(InWorkerIndex))
	{
		Workers[InWorkerIndex]->AbortCurrentTakeUpload();
		return true;
	}

	return false;
}

FGuid FLiveLinkHubExportManager::GetClientId() const
{
	return ClientId;
}

int32 FLiveLinkHubExportManager::GetPreferredWorkerIndex() const
{
	int32 MinimalTaskCounter = INT32_MAX;
	int32 WorkerIdWithMinimalCounter = 0;

	for (int32 WorkerId = 0; WorkerId < Workers.Num(); ++WorkerId)
	{
		int32 WorkerTaskCounter = Workers[WorkerId]->GetTaskCount();

		if (MinimalTaskCounter > WorkerTaskCounter)
		{
			MinimalTaskCounter = WorkerTaskCounter;
			WorkerIdWithMinimalCounter = WorkerId;
		}
	}

	return WorkerIdWithMinimalCounter;
}