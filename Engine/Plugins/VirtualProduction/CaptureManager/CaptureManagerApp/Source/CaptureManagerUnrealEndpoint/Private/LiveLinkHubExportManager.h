// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHubExportClient.h"

class FLiveLinkHubExportManager
{
public:

	FLiveLinkHubExportManager(FGuid InClientId,
							  FLiveLinkHubExportClient::FOnDataUploaded InDataUploaded,
							  uint64 InDefaultNumberOfWorkers = DefaultNumberOfWorkers);

	~FLiveLinkHubExportManager();

	[[nodiscard]] int32 ExportTake(const FLiveLinkHubExportClient::FTakeUploadParams& InTakeUploadParams,
								   const FString& InTakeDirectory,
								   const FTakeMetadata& InTakeMetadata);

	bool AbortExport(int32 InWorkerId);

	FGuid GetClientId() const;

private:

	static const uint64 DefaultNumberOfWorkers = 2;

	int32 GetPreferredWorkerIndex() const;

	FGuid ClientId;

	FCriticalSection Mutex;
	TArray<TSharedPtr<FLiveLinkHubExportClient>> Workers;
};
