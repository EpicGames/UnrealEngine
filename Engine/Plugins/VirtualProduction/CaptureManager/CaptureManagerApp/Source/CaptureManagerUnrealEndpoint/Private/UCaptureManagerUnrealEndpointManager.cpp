// Copyright Epic Games, Inc. All Rights Reserved.

#include "UCaptureManagerUnrealEndpointManager.h"

#include "CaptureManagerUnrealEndpointModule.h"

struct UCaptureManagerUnrealEndpointManager::FImpl
{
	FImpl();

	void Start();
	void Stop();

	TSharedRef<UE::CaptureManager::FUnrealEndpointManager> EndpointManager;
};

UCaptureManagerUnrealEndpointManager::UCaptureManagerUnrealEndpointManager() :
	Impl(MakePimpl<FImpl>())
{
}

UCaptureManagerUnrealEndpointManager::~UCaptureManagerUnrealEndpointManager() = default;

void UCaptureManagerUnrealEndpointManager::Start()
{
	Impl->Start();
}

void UCaptureManagerUnrealEndpointManager::Stop()
{
	Impl->Stop();
}

bool UCaptureManagerUnrealEndpointManager::WaitForEndpointByHostName(const FString& InHostName, const int32 InTimeoutMS)
{
	using namespace UE::CaptureManager;

	TOptional<TWeakPtr<FUnrealEndpoint>> NativeEndpoint = Impl->EndpointManager->WaitForEndpoint(
		[&InHostName](const FUnrealEndpoint& InEndpoint)
		{
			return InEndpoint.GetInfo().HostName == InHostName;
		},
		InTimeoutMS
	);

	return NativeEndpoint.IsSet();
}

UCaptureManagerUnrealEndpointManager::FImpl::FImpl() :
	EndpointManager(FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager())
{
}

void UCaptureManagerUnrealEndpointManager::FImpl::Start()
{
	EndpointManager->Start();
}

void UCaptureManagerUnrealEndpointManager::FImpl::Stop()
{
	EndpointManager->Stop();
}
