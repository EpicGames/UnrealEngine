// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/PimplPtr.h"

#include "UCaptureManagerUnrealEndpointManager.generated.h"

// This class wraps an underlying Unreal Endpoint Manager so that python and blueprints can make use of it.
UCLASS(BlueprintType)
class UCaptureManagerUnrealEndpointManager : public UObject
{
	GENERATED_BODY()

public:
	UCaptureManagerUnrealEndpointManager();
	~UCaptureManagerUnrealEndpointManager();

	// Starts the endpoint manager and the discovery of unreal endpoints.
	UFUNCTION(BlueprintCallable, Category = "LiveLinkHub|CaptureManager|UnrealEndpointManager")
	void Start();

	// Stops the endpoint manager.
	UFUNCTION(BlueprintCallable, Category = "LiveLinkHub|CaptureManager|UnrealEndpointManager")
	void Stop();

	// Waits for an endpoint with a particular host name to be discovered or a timeout is reached.
	// returns true if the host was found, false if the timeout was exceeded
	UFUNCTION(BlueprintCallable, Category = "LiveLinkHub|CaptureManager|UnrealEndpointManager")
	bool WaitForEndpointByHostName(const FString& InHostName, int32 InTimeoutMS);

private:
	struct FImpl;
	TPimplPtr<FImpl> Impl;
};
