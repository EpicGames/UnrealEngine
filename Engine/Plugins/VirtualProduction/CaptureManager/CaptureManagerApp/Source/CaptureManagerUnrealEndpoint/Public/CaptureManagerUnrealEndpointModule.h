// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerUnrealEndpointManager.h"

#include "Modules/ModuleManager.h"

class CAPTUREMANAGERUNREALENDPOINT_API FCaptureManagerUnrealEndpointModule 
	: public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	TSharedRef<UE::CaptureManager::FUnrealEndpointManager> GetEndpointManager() const;
	TOptional<TSharedRef<UE::CaptureManager::FUnrealEndpointManager>> GetEndpointManagerIfValid() const;

private:
	TSharedPtr<UE::CaptureManager::FUnrealEndpointManager> EndpointManager;
};

