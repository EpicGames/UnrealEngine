// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ExampleLiveLinkDevices : ModuleRules
{
	public ExampleLiveLinkDevices(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"LiveLinkDevice",
			"IngestLiveLinkDevice",
			"LiveLinkCapabilities",
			"CaptureManagerTakeMetadata",
			"CoreUObject",
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
