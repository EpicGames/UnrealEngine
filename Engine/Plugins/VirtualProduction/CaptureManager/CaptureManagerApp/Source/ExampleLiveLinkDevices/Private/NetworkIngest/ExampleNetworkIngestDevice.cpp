// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleNetworkIngestDevice.h"

#include "Async/TaskProgress.h"

const UExampleNetworkIngestDeviceSettings* UExampleNetworkIngestDevice::GetSettings() const
{
	return GetDeviceSettings<UExampleNetworkIngestDeviceSettings>();
}

TSubclassOf<ULiveLinkDeviceSettings> UExampleNetworkIngestDevice::GetSettingsClass() const
{
	return UExampleNetworkIngestDeviceSettings::StaticClass();
}

FText UExampleNetworkIngestDevice::GetDisplayName() const
{
	return FText::FromString(GetSettings()->DisplayName);
}

EDeviceHealth UExampleNetworkIngestDevice::GetDeviceHealth() const
{
	return EDeviceHealth::Nominal;
}

FText UExampleNetworkIngestDevice::GetHealthText() const
{
	return FText::FromString("Example Health");
}

void UExampleNetworkIngestDevice::OnDeviceAdded()
{
	// Respond to device added event here

	Super::OnDeviceAdded();
}

void UExampleNetworkIngestDevice::OnDeviceRemoved()
{
	// Respond to device removed event here
	//
	// For example,execute disconnect on the connection capability:
	// 
	// ILiveLinkDeviceCapability_Connection::Execute_Disconnect(this);

	Super::OnDeviceRemoved();
}

void UExampleNetworkIngestDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	// Respond to any settings changes here	

	Super::OnSettingChanged(InPropertyChangedEvent);
}

FString UExampleNetworkIngestDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
	// Get the full file path to the take on disk here
	//   See ULiveLinkFaceDevice::GetFullTakePath

	return FString();
}

void UExampleNetworkIngestDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	// Query your device for a list of takes
	// Populate an FTakeMetadata object per take
	// Call IngestCapability->AddTake

	// For example:
	//
	// TArray<FTakeMetadata> Takes = Device.Takes();
	// for (const FTakeMetadata& Take : Takes)
	// {
	// 	 int32 TakeId = AddTake(Take);
	// 	 TakesById[TakeId] = Take;
	// }

	// See ULiveLinkFaceDevice::UpdateTakeList_Implementation
}

void UExampleNetworkIngestDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	// Get the take corresponding to the InTakeId from your device (i.e. Download the take from the device)
	//     See ULiveLinkFaceDevice::RunIngestTake
	
	static constexpr uint32 NumberOfTasks = 3; // Download, Convert, Upload
	// Download - handled by you
	// Convert + Upload - handled by UBaseIngestLiveLinkDevice

	using namespace UE::CaptureManager;
	TSharedPtr<FTaskProgress> TaskProgress = MakeShared<FTaskProgress>
		(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([this, InProcessHandle](double InProgress)
			{
				ExecuteProcessProgressReporter(InProcessHandle, InProgress);
			}));

	// Note: IngestTake uses GetFullTakePath() so make sure you have implemented it

	Super::IngestTake(InProcessHandle, InIngestOptions, MoveTemp(TaskProgress));
}

void UExampleNetworkIngestDevice::CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle)
{
	// Stop your device from sending data here

	Super::CancelIngest(InProcessHandle->GetTakeId());
}

ELiveLinkDeviceConnectionStatus UExampleNetworkIngestDevice::GetConnectionStatus_Implementation() const
{
	// Query your device for its connection status here

	return ELiveLinkDeviceConnectionStatus::Disconnected;
}

FString UExampleNetworkIngestDevice::GetHardwareId_Implementation() const
{
	return TEXT("Example Device ID");
}

bool UExampleNetworkIngestDevice::SetHardwareId_Implementation(const FString& HardwareID)
{
	return false;
}

bool UExampleNetworkIngestDevice::Connect_Implementation()
{
	// Connect your device here

	return true;
}

bool UExampleNetworkIngestDevice::Disconnect_Implementation()
{
	// Disconnect your device here
	
	return true;
}

bool UExampleNetworkIngestDevice::StartRecording_Implementation()
{
	// Start recording on your device here

	// You can get the current slate and take information from the live link recording session
	// 
	// Example:
	// 
	// ILiveLinkRecordingSessionInfo& SessionInfo = ILiveLinkRecordingSessionInfo::Get();
	// Device->StartRecording(SessionInfo);

	return false;
}

bool UExampleNetworkIngestDevice::StopRecording_Implementation()
{
	// Stop recording on your device here

	return false;
}

bool UExampleNetworkIngestDevice::IsRecording_Implementation() const
{
	// Query your device for its recording state here

	return false;
}
