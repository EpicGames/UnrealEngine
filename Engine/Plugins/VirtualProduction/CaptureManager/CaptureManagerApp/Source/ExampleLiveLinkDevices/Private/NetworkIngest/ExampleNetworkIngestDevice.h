// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseIngestLiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"

#include "Internationalization/Text.h"
#include "Templates/SubclassOf.h"

#include "ExampleNetworkIngestDevice.generated.h"


UCLASS()
class UExampleNetworkIngestDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:

	UExampleNetworkIngestDeviceSettings()
	{
		DisplayName = TEXT("Example Network Ingest Device");
	}
	
	UPROPERTY(EditAnywhere, Category = "Example Network Ingest Device")
	FString IpAddress;
	
	UPROPERTY(EditAnywhere, Category = "Example Network Ingest Device")
	uint16 Port = 14785;
};


UCLASS()
class UExampleNetworkIngestDevice : public UBaseIngestLiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
{
	GENERATED_BODY()

public:
	const UExampleNetworkIngestDeviceSettings* GetSettings() const;

	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual FText GetDisplayName() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

private:
	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;

	//~ Begin ULiveLinkDeviceCapability_Ingest interface
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle,
							   const UIngestCapability_Options* InIngestOptions) override;
	virtual void CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle) override;
	//~ End ULiveLinkDeviceCapability_Ingest interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	//~ Begin ILiveLinkDeviceCapability_Recording interface
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface
};
