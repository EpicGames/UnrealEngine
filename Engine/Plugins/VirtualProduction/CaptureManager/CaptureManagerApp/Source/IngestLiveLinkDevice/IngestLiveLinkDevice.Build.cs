// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class IngestLiveLinkDevice : ModuleRules
{
	public IngestLiveLinkDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "IngestLiveLinkDevice";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core",
			"MediaAssets"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureProtocolStack",
			"CaptureDataUtils",
			"CaptureDataConverter",
			"CaptureManagerMediaRW",
			"CaptureManagerUnrealEndpoint",
			"CaptureUtils",
			"Engine",
			"Json",
			"LiveLinkDevice",
			"LiveLinkCapabilities",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"LiveLinkHub",
			"ToolWidgets",
			"Slate",
			"SlateCore",
			"CaptureManagerTakeMetadata",
			"CaptureManagerSettings",
			"CoreUObject",
			"UnrealEd",
			"ElectraPlayerPlugin",
			"MJPEGDecoderElectra",
			"NamingTokens"
		});

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
