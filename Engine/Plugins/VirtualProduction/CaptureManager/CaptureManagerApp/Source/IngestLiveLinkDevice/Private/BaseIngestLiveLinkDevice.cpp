// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseIngestLiveLinkDevice.h"

#include "HAL/FileManager.h"
#include "CaptureManagerUnrealEndpointModule.h"
#include "Settings/CaptureManagerSettings.h"
#include "Settings/CaptureManagerTemplateTokens.h"
#include "CaptureManagerEncoderNamingTokens.h"

#include "Utils/IngestLiveLinkDeviceUtils.h"
#include "Utils/IngestLiveLinkDeviceLog.h"

#include "CaptureDataConverter.h"
#include "Engine/Engine.h"

#include "NamingTokensEngineSubsystem.h"
#include "Async/TaskWaiter.h"

DEFINE_LOG_CATEGORY(LogIngestLiveLinkDevice);

namespace UE::CaptureManager::Private
{

static FString ResolveCommandArgs(const FStringView InDefaultArgs, const FString& InCustomArgs, const FString& InTokenNamespace)
{
	FString CommandArgs = InCustomArgs.IsEmpty() ? FString(InDefaultArgs) : InCustomArgs;

	if (!GEngine)
	{
		UE_LOGF(LogIngestLiveLinkDevice, Warning, "Cannot resolve encoder command tokens: GEngine is unavailable");
		return CommandArgs;
	}

	UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();

	if (!NamingTokensSubsystem)
	{
		UE_LOGF(LogIngestLiveLinkDevice, Warning, "Cannot resolve encoder command tokens: NamingTokensSubsystem is unavailable");
		return CommandArgs;
	}

	FNamingTokenFilterArgs FilterArgs;
	FilterArgs.AdditionalNamespacesToInclude.Add(InTokenNamespace);
	FilterArgs.bNativeOnly = true;

	CommandArgs = NamingTokensSubsystem->EvaluateTokenString(CommandArgs, FilterArgs).EvaluatedText.ToString();

	return CommandArgs;
}

static void ApplyThirdPartyEncoderSettings(const UCaptureManagerSettings* InSettings, FCaptureDataConverterParams& OutParams)
{
	if (!InSettings || !InSettings->bEnableThirdPartyEncoder || InSettings->ThirdPartyEncoder.FilePath.IsEmpty())
	{
		return;
	}

	const bool bThirdPartyEncoderIsAvailable = FPlatformProcess::ExecProcess(*InSettings->ThirdPartyEncoder.FilePath, TEXT("-version"), nullptr, nullptr, nullptr);

	if (bThirdPartyEncoderIsAvailable)
	{
		// {input}, {output} and {params} tokens will remain and are resolved with an FString::Format in the encoder command's Execute() function
		if (const UCaptureManagerAudioEncoderTokens* AudioTokens = InSettings->GetAudioEncoderNamingTokens())
		{
			const FString CommandArgs = ResolveCommandArgs(UE::CaptureManager::EncoderDefaults::AudioCommandArgs, InSettings->CustomAudioCommandArguments, AudioTokens->GetNamespace());
			OutParams.AudioEncoderConfig = FCaptureManagerEncoderConfig{InSettings->ThirdPartyEncoder.FilePath, CommandArgs};
		}

		if (const UCaptureManagerVideoEncoderTokens* VideoTokens = InSettings->GetVideoEncoderNamingTokens())
		{
			const FString CommandArgs = ResolveCommandArgs(UE::CaptureManager::EncoderDefaults::VideoCommandArgs, InSettings->CustomVideoCommandArguments, VideoTokens->GetNamespace());
			OutParams.VideoEncoderConfig = FCaptureManagerEncoderConfig{InSettings->ThirdPartyEncoder.FilePath, CommandArgs};
		}
	}
}

static void TryCleanUpWorkingDirectory(const FString& InWorkingDirectory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBaseIngestDevice_CleanUpWorkingDirectory);

	FString NormalizedFullPath = FPaths::ConvertRelativePathToFull(InWorkingDirectory);
	FPaths::NormalizeFilename(NormalizedFullPath);

	// Note: The raw input string is used for the empty check, not the normalized full path, as we are basically 
	// trying to catch the situation when someone has forgotten to set it, before data loss occurs.

	if (InWorkingDirectory.IsEmpty())
	{
		UE_LOGF(
			LogIngestLiveLinkDevice,
			Error,
			"Refused to clean up the ingest working directory. The supplied path was an empty string, which could "
				"have resulted in the unintended removal of the current working directory: %ls",
			*NormalizedFullPath
		);

		return;
	}

	const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>();
	check(CaptureManagerSettings);

	if (CaptureManagerSettings->bShouldCleanWorkingDirectory)
	{
		UE_LOGF(LogIngestLiveLinkDevice, Display, "Deleting directory: %ls", *NormalizedFullPath);

		constexpr bool bRequireExists = false;
		constexpr bool bIsRecursive = true;
		const bool bIsDeleted = IFileManager::Get().DeleteDirectory(*NormalizedFullPath, bRequireExists, bIsRecursive);

		if (bIsDeleted)
		{
			UE_LOGF(LogIngestLiveLinkDevice, Display, "Directory deleted: %ls", *NormalizedFullPath);
		}
		else
		{
			const uint32 ErrorCode = FPlatformMisc::GetLastError();
			UE_LOGF(LogIngestLiveLinkDevice, Error, "Failed to delete directory (Error code: %u): %ls", ErrorCode, *NormalizedFullPath);
		}
	}
}

/** Convenience RAII object to clean up the working area when the scope ends - it just saves us forgetting to do it */
class FScopedWorkingArea
{
public:
	FScopedWorkingArea(FString InDirectoryPath) :
		DirectoryPath(MoveTemp(InDirectoryPath))
	{
	}

	~FScopedWorkingArea()
	{
		TryCleanUpWorkingDirectory(DirectoryPath);
	}

private:
	const FString DirectoryPath;
};

} // namespace UE::CaptureManager::Private

class UBaseIngestLiveLinkDevice::FImpl : public TSharedFromThis<UBaseIngestLiveLinkDevice::FImpl>
{
public:

	explicit FImpl(UBaseIngestLiveLinkDevice* InOwner);

	void IngestTake(const UIngestCapability_ProcessHandle* InProcessHandle, 
					const UIngestCapability_Options* InIngestOptions, 
					TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress);
	void CancelIngest(int32 InTakeId);
	void OnDeviceRemoved(TArray<int32> TakeIdentifiers);

private:

	FCaptureDataConverterResult<void> RunConversion(int32 InTakeId,
													TStrongObjectPtr<const UIngestCapability_Options> InIngestOptions,
													TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress,
													FCaptureDataConverterParams& InParams);

	struct FTakeUploadInfo
	{
		FGuid TaskId;
		FGuid EndpointID;
	};

	FCriticalSection Mutex;

	// We hold a strong ptr to the parent UBaseIngestLiveLinkDevice, because this impl object could otherwise outlive the parent (it is shared with callbacks)
	TStrongObjectPtr<UBaseIngestLiveLinkDevice> Owner;
	TMap<int32, TSharedRef<FCaptureDataConverter>> Converters;
	TMap<int32, FTakeUploadInfo> TakeUploads;

	// Every invocation of `IngestTake` function will increase a counter in TaskWaiter, registering that task is running.
	// Once function finishes it's work, counter is decremented.
	// And if it happens that device is being removed before counter is equal to zero, TaskWaiter will busy wait for it to reach zero.
	// Ensures that 'IngestTake' does not access potentially destroyed device.
	UE::CaptureManager::FTaskWaiter TaskWaiter;
};

UBaseIngestLiveLinkDevice::FImpl::FImpl(UBaseIngestLiveLinkDevice* InOwner) :
	Owner(InOwner)
{
}

void UBaseIngestLiveLinkDevice::FImpl::OnDeviceRemoved(TArray<int32> TakeIdentifiers)
{	
	for (const int32 TakeIdentifier : TakeIdentifiers)
	{
		CancelIngest(TakeIdentifier);
	}

	while (true)
	{
		FScopeLock TakeUploadsRemoveLock(&Mutex);
		int32 TakesInProgressCount = TakeUploads.Num();
		TakeUploadsRemoveLock.Unlock();

		if (TakesInProgressCount == 0)
		{
			break;
		}
		FPlatformProcess::Sleep(0.1f);
	}

	TaskWaiter.WaitForAll();
}

void UBaseIngestLiveLinkDevice::FImpl::IngestTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions, TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBaseIngestLiveLinkDevice_IngestTake);

	if (!TaskWaiter.CreateTask())
	{
		Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::InternalError, TEXT("Failed to create task waiter")));
		return;
	}

	ON_SCOPE_EXIT
	{
		TaskWaiter.FinishTask();
	};

	TStrongObjectPtr<const UIngestCapability_Options> IngestOptions(InIngestOptions);

	using namespace UE::CaptureManager;
	using namespace UE::CaptureManager::Private;

	int32 TakeId = InProcessHandle->GetTakeId();
	FString TakeFullPath = Owner->GetFullTakePath(TakeId);

	if (TakeFullPath.IsEmpty())
	{
		FString Message = FString::Printf(TEXT("Failed to lookup take directory for take id %d"), TakeId);
		ensureMsgf(false, TEXT("%s"), *Message);
		Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::InvalidArgument, MoveTemp(Message)));
		return;
	}

	TOptional<FTakeMetadata> MaybeTakeMetadata = Owner->GetTakeMetadata(TakeId);
	if (!MaybeTakeMetadata.IsSet())
	{
		FString Message = FString::Printf(TEXT("Failed to lookup take metadata for take id %d (TakePath=%s)"), TakeId, *TakeFullPath);
		UE_LOGF(LogIngestLiveLinkDevice, Error, "%ls", *Message);
		Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::InvalidArgument, MoveTemp(Message)));
		return;
	}

	FTakeMetadata TakeMetadata = MoveTemp(MaybeTakeMetadata.GetValue());

	FString TakeName = TakeMetadata.Slate + TEXT("_") + FString::FromInt(TakeMetadata.TakeNumber);

	const FString TakeConversionDirectory = IngestOptions->WorkingDirectory;

	FGuid TakeUploadId = FGuid::NewGuid();

	FCaptureDataConverterParams Params;
	Params.TakeName = TakeName;
	Params.TakeMetadata = TakeMetadata;
	Params.TakeOriginDirectory = MoveTemp(TakeFullPath);
	Params.TakeOutputDirectory = FPaths::Combine(TakeConversionDirectory, TakeName + TEXT("_") + TakeUploadId.ToString());

	FCaptureDataConverterResult<void> Result = RunConversion(TakeId, IngestOptions, InTaskProgress, Params);

	{
		FScopeLock ConvertersRemoveLock(&Mutex);
		Converters.Remove(TakeId);
	}

	// Shared just so we can pass it into the  completion callback without invoking the destructor (can't use unique due to branch structure below)
	TSharedRef<FScopedWorkingArea> ScopedWorkingArea = MakeShared<FScopedWorkingArea>(Params.TakeOutputDirectory);

	if (Result.HasValue())
	{
		if (IngestOptions->UploadHostName.IsEmpty())
		{
			FString Message = TEXT("The upload endpoint was not specified (it was an empty string), try setting a default in the hub settings");
			Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::UnrealEndpointNotFound, MoveTemp(Message)));
			return;
		}

		TSharedRef<FUnrealEndpointManager> UnrealEndpointManager = FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager();

		TOptional<TWeakPtr<FUnrealEndpoint>> WeakUnrealEndpoint = UnrealEndpointManager->FindEndpointByPredicate(
			[&IngestOptions](const FUnrealEndpoint& InEndpoint)
			{
				return InEndpoint.GetInfo().HostName == IngestOptions->UploadHostName;
			}
		);

		// The endpoint was not found at all
		if (!WeakUnrealEndpoint)
		{
			FString Message = FString::Printf(
				TEXT("Upload failed because the requested endpoint was not found (check the ingest server is running in UE): %s"),
				*IngestOptions->UploadHostName
			);
			Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::UnrealEndpointNotFound, MoveTemp(Message)));
			return;
		}

		TSharedPtr<FUnrealEndpoint> UnrealEndpoint = WeakUnrealEndpoint->Pin();

		// The endpoint was found but has since expired
		if (!UnrealEndpoint)
		{
			FString Message = FString::Printf(TEXT("Upload failed because the requested endpoint has expired: %s"), *IngestOptions->UploadHostName);
			Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::UnrealEndpointNotFound, MoveTemp(Message)));
			return;
		}

		// We wait until after the download and conversion to start the connection, in this way we won't require a
		// connection for download only workflows. Note: Disconnection occurs for all endpoints at the end of job
		// processing (in the ingest job processor).
		UnrealEndpoint->StartConnection();

		// Make sure we're not blocking the game thread. This wait should only be noticeable for the first task to
		// connect to a particular endpoint, after that this wait should be negligible for that endpoint.
		check(!IsInGameThread());
		const bool bIsConnected = UnrealEndpoint->WaitForConnectionState(FUnrealEndpoint::EConnectionState::Connected, 3'000);

		if (!bIsConnected)
		{
			FString EndpointInfoString = UnrealEndpointInfoToString(UnrealEndpoint->GetInfo());
			FString Message = FString::Printf(TEXT("Upload failed because we timed out connecting to the endpoint: %s"), *EndpointInfoString);
			Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::UnrealEndpointConnectionTimedOut, MoveTemp(Message)));
			return;
		}

		TUniquePtr<FTakeUploadTask> TakeUploadTask = MakeUnique<FTakeUploadTask>(
			TakeUploadId,
			Owner->GetDeviceId(),
			Owner->GetDisplayName().ToString(),
			Params.TakeOutputDirectory,
			TakeMetadata
		);

		TakeUploadTask->Progressed().BindLambda(
			// We need to keep a link back to the task progress object, as the upload task refers to it internally
			[TaskProgress = InTaskProgress, UploadTask = InTaskProgress->StartTask()](const double InProgressed) mutable
			{
				UploadTask.Update(InProgressed);
			}
		);

		TakeUploadTask->Complete().BindLambda(
			// We need to keep the Impl alive until the task is complete, as we need to use it
			[Impl = SharedThis(this), ProcessHandle = InProcessHandle, TakeOutputDirectory = Params.TakeOutputDirectory, ScopedWorkingArea = ScopedWorkingArea](const FString& InMessage, const int32 InStatusCode)
			{
				FScopeLock TakeUploadsRemoveLock(&Impl->Mutex);
				Impl->TakeUploads.Remove(ProcessHandle->GetTakeId());
				TakeUploadsRemoveLock.Unlock();

				if (InStatusCode != 0 || !InMessage.IsEmpty())
				{
					TValueOrError<void, FIngestCapability_Error> Err = MakeError(FIngestCapability_Error::UnrealEndpointUploadError, InMessage);
					Impl->Owner->ExecuteProcessFinishedReporter(ProcessHandle, MoveTemp(Err));
					return;
				}

				Impl->Owner->ExecuteProcessFinishedReporter(ProcessHandle, MakeValue<>());
			}
		);

		FGuid TaskID = TakeUploadTask->GetTaskID();
		const bool bAddOk = UnrealEndpoint->AddTakeUploadTask(MoveTemp(TakeUploadTask));

		if (bAddOk)
		{
			// Add an entry to the lookup table used for aborting queued upload tasks
			FTakeUploadInfo TakeUploadInfo;
			TakeUploadInfo.TaskId = MoveTemp(TaskID);
			TakeUploadInfo.EndpointID = UnrealEndpoint->GetInfo().EndpointID;

			FScopeLock TakeUploadsEmplaceLock(&Mutex);
			TakeUploads.Emplace(InProcessHandle->GetTakeId(), MoveTemp(TakeUploadInfo));
		}
		else
		{
			FString Message = FString::Printf(TEXT("Failed to add upload task: %s"), *TaskID.ToString());
			Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::InternalError, MoveTemp(Message)));
		}
	}
	else
	{
		FCaptureDataConverterError Error = Result.StealError();

		FText Message = FText::Join(FText::FromString(TEXT("\n\r")), Error.GetErrors());
		Owner->ExecuteProcessFinishedReporter(InProcessHandle, MakeError(FIngestCapability_Error::ConversionError, Message.ToString()));
	}
}

void UBaseIngestLiveLinkDevice::FImpl::CancelIngest(int32 InTakeId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBaseIngestLiveLinkDevice_CancelIngest);

	using namespace UE::CaptureManager;

	FScopeLock ScopeLock(&Mutex);

	if (TSharedRef<FCaptureDataConverter>* Converter = Converters.Find(InTakeId))
	{
		(*Converter)->Cancel();
	}

	const FTakeUploadInfo* TakeUploadInfo = TakeUploads.Find(InTakeId);

	if (TakeUploadInfo)
	{
		// Find the endpoint responsible for uploading this task
		TSharedRef<FUnrealEndpointManager> UnrealEndpointManager = FModuleManager::LoadModuleChecked<FCaptureManagerUnrealEndpointModule>("CaptureManagerUnrealEndpoint").GetEndpointManager();

		TOptional<TWeakPtr<FUnrealEndpoint>> WeakUnrealEndpoint = UnrealEndpointManager->FindEndpointByPredicate(
			[TakeUploadInfo](const FUnrealEndpoint& InEndpoint) -> bool
			{
				return InEndpoint.GetInfo().EndpointID == TakeUploadInfo->EndpointID;
			}
		);

		if (WeakUnrealEndpoint)
		{
			if (const TSharedPtr<FUnrealEndpoint> UnrealEndpoint = WeakUnrealEndpoint->Pin())
			{
				UnrealEndpoint->CancelTakeUploadTask(TakeUploadInfo->TaskId);
			}
		}
	}
}

FCaptureDataConverterResult<void> UBaseIngestLiveLinkDevice::FImpl::RunConversion(int32 InTakeId,
																				  TStrongObjectPtr<const UIngestCapability_Options> InIngestOptions,
																				  TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress,
																				  FCaptureDataConverterParams& InParams)
{
	using namespace UE::CaptureManager;

	TRACE_CPUPROFILER_EVENT_SCOPE(FBaseIngestLiveLinkDevice_RunConversion);

	FTaskProgress::FTask ConvertTask = InTaskProgress->StartTask();

	FCaptureConvertVideoOutputParams VideoParams;
	VideoParams.Format = InIngestOptions->Video.Format;
	VideoParams.ImageFileName = InIngestOptions->Video.FileNamePrefix;
	VideoParams.OutputPixelFormat = ToMediaPixelFormat(InIngestOptions->Video.PixelFormat);
	VideoParams.Rotation = ToMediaOrientation(InIngestOptions->Video.Rotation);

	InParams.VideoOutputParams = MoveTemp(VideoParams);

	FCaptureConvertAudioOutputParams AudioParams;
	AudioParams.Format = InIngestOptions->Audio.Format;
	AudioParams.AudioFileName = InIngestOptions->Audio.FileNamePrefix;

	InParams.AudioOutputParams = MoveTemp(AudioParams);

	FCaptureConvertDepthOutputParams DepthParams;
	DepthParams.ImageFileName = TEXT("depth");
	DepthParams.Rotation = ToMediaOrientation(InIngestOptions->Video.Rotation); // Apply the same rotation as video to depth stream

	InParams.DepthOutputParams = MoveTemp(DepthParams);

	FCaptureConvertCalibrationOutputParams CalibrationParams;
	CalibrationParams.FileName = TEXT("calibration");

	InParams.CalibrationOutputParams = MoveTemp(CalibrationParams);

	if (const UCaptureManagerSettings* CaptureManagerSettings = GetDefault<UCaptureManagerSettings>())
	{
		Private::ApplyThirdPartyEncoderSettings(CaptureManagerSettings, InParams);
	}

	FCaptureDataConverter::FProgressReporter ProgressReporter =
		FCaptureDataConverter::FProgressReporter::CreateLambda([ConvertTask](const double InProgress) mutable
	{
		ConvertTask.Update(InProgress);
	});

	FScopeLock ScopeLock(&Mutex);
	TSharedRef<FCaptureDataConverter> Converter = Converters.FindOrAdd(InTakeId, MakeShared<FCaptureDataConverter>());
	ScopeLock.Unlock();

	// Blocking function so do not hold the lock for this
	return Converter->Run(InParams, MoveTemp(ProgressReporter));
}

void UBaseIngestLiveLinkDevice::OnDeviceAdded()
{
	Impl = MakeShared<FImpl>(this);
}

void UBaseIngestLiveLinkDevice::OnDeviceRemoved()
{
	Execute_RequestAbortUpdateTakeList(this);

	Impl->OnDeviceRemoved(Execute_GetTakeIdentifiers(this));

	RemoveAllTakes();

	Super::OnDeviceRemoved();
}

void UBaseIngestLiveLinkDevice::IngestTake(const UIngestCapability_ProcessHandle* InProcessHandle, 
										   const UIngestCapability_Options* InIngestOptions, 
										   TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress)
{
	Impl->IngestTake(InProcessHandle, InIngestOptions, MoveTemp(InTaskProgress));
}

void UBaseIngestLiveLinkDevice::RunDownloadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
	// No download needed so this step is always a success
	ExecuteProcessProgressReporter(InProcessHandle, 1.f);
	ExecuteProcessFinishedReporter(InProcessHandle, MakeValue());
}

void UBaseIngestLiveLinkDevice::CancelIngest(int32 InTakeId)
{
	Impl->CancelIngest(InTakeId);
}

void UBaseIngestLiveLinkDevice::CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle)
{
	Impl->CancelIngest(InProcessHandle->GetTakeId());
}
