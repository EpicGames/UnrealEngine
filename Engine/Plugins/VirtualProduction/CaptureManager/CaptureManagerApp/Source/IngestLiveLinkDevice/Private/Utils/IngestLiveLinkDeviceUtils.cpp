// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/IngestLiveLinkDeviceUtils.h"

namespace UE::CaptureManager
{

FString ErrorOriginToString(FTakeMetadataParserError::EOrigin InOrigin)
{
	switch (InOrigin)
	{
		case FTakeMetadataParserError::Reader:
			return TEXT("Reader");
		case FTakeMetadataParserError::Validator:
			return TEXT("Validator");
		case FTakeMetadataParserError::Parser:
		default:
			return TEXT("Parser");
	}
}

}