// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "Async/TaskProgress.h"
#include "Misc/Optional.h"

#include "BaseIngestLiveLinkDevice.generated.h"

/**
 * Base class that provides default implementations of core ingest capability functions.
 */
UCLASS(Abstract)
class INGESTLIVELINKDEVICE_API UBaseIngestLiveLinkDevice : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_Ingest
{
	GENERATED_BODY()

public:

	//~ Begin ULiveLinkDevice interface

	/**
	 * Inherited function from the ULiveLinkDevice that is called when device is added
	 * @note Users are required to invoke this function if it is inherited in the users code.
	 */
	virtual void OnDeviceAdded() override;

	/**
	 * Inherited function from the ULiveLinkDevice that is called when device is removed
	 * @note Users are required to invoke this function if it is inherited in the users code.
	 */
	virtual void OnDeviceRemoved() override;

	//~ End ULiveLinkDevice interface

protected:

	/**
	 * Function implements the default behavior for conversion and upload to the UE of the take data
	 * @params ProcessHandle Pointer to the process handle created for this process
	 * @params IngestOptions Ingest options used for conversion process
	 * @params TaskProgress Helper class that accumulates progress and reports the progress to the user
	 */
	void IngestTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions, TSharedPtr<UE::CaptureManager::FTaskProgress> InTaskProgress);

	/**
	 * Pure virtual function that returns the full path to the take data. Full path to the take data is required for the conversion step.
	 * @params TakeId Take identifier
	 * @returns Full path to the take data
	 */
	virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const PURE_VIRTUAL(UBaseIngestLiveLinkDevice::GetFullTakePath, return FString(););

	//~ Begin ULiveLinkDeviceCapability_Ingest interface

	/**
	 * Inherited function from the ILiveLinkDeviceCapability_Ingest. The function is required to be implemented.
	 * @note In it's implementation, user may choose to call IngestTake function or to implement custom download step
	 * @params ProcessHandle Pointer to the process handle created for this process
	 * @params IngestOptions Ingest options used for conversion process
	 */
	virtual void RunDownloadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;

	/**
	 * Inherited function from the ILiveLinkDeviceCapability_Ingest. The function is required to be implemented.
	 * @note In it's implementation, user may choose to call IngestTake function or to implement custom conversion and upload step
	 * @params ProcessHandle Pointer to the process handle created for this process
	 * @params IngestOptions Ingest options used for conversion process
	 */
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override PURE_VIRTUAL(UBaseIngestLiveLinkDevice::RunConvertAndUploadTake);

	/**
	 * Inherited function from the ILiveLinkDeviceCapability_Ingest. The function is required to be implemented.
	 * @note Runs update take list in the current thread
	 * @param Callback to be run
	 */
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override PURE_VIRTUAL(UBaseIngestLiveLinkDevice::RunUpdateTakeList);

	//~ End ULiveLinkDeviceCapability_Ingest interface

	/**
	 * Inherited function from the ILiveLinkDeviceCapability_Ingest. The function is required to be implemented.
	 * @note In it's default implementation, it will abort the default ingest process.
	 * @params TakeId Take identifier
	 */
	void CancelIngest(int32 InTakeId);


private:

	virtual void CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle) override;

	class FImpl;
	TSharedPtr<FImpl> Impl;
};

