// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureManagerTakeMetadata.h"

namespace UE::CaptureManager
{

INGESTLIVELINKDEVICE_API FString ErrorOriginToString(FTakeMetadataParserError::EOrigin InOrigin);

}
