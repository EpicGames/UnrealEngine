// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkCapabilities : ModuleRules
{
	public LiveLinkCapabilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "LiveLinkCapabilities";

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"CaptureUtils",
			"Core",
			"CaptureManagerUnrealEndpoint", // TODO: Only needed for the ULiveLinkHubIngestClientInfo definition, move it and remove this dependency
			"CaptureManagerTakeMetadata",
			"CaptureDataConverter"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CaptureManagerMediaRW",
			"Json",
			"LiveLinkDevice",
			"Media",
			"LiveLinkHubCaptureMessaging",
			"SlateCore",
			"Slate",
			"CoreUObject",
			"MediaAssets",
			"MediaFrameworkUtilities",
			"MediaProfile",
			"Engine"
		});

		if (Target.bBuildEditor) 
		{ 
			PrivateDependencyModuleNames.Add("UnrealEd"); 
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
