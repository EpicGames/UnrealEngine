// Copyright Epic Games, Inc. All Rights Reserved.

#include "Ingest/IngestCapability_ProcessHandle.h"

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

namespace UE::CaptureManager
{
static int32 GetNumberOfProcessTasks(EIngestCapability_ProcessConfig InProcessConfig)
{
	switch (InProcessConfig)
	{
		case EIngestCapability_ProcessConfig::Ingest:
			return 2; // Download + Convert&Upload
		case EIngestCapability_ProcessConfig::Download:
			return 1;
		case EIngestCapability_ProcessConfig::None:
		default:
			check(false);
			return 0;
	}
}
}

FIngestCapability_Error::FIngestCapability_Error(ECode InCode, FString InMessage)
	: Code(InCode)
	, Message(MoveTemp(InMessage))
{
}

FIngestCapability_Error::ECode FIngestCapability_Error::GetCode() const
{
	return Code;
}

const FString& FIngestCapability_Error::GetMessage() const
{
	return Message;
}

FIngestCapability_ProcessContext::FIngestCapability_ProcessContext(int32 InTakeId,
																   EIngestCapability_ProcessConfig InProcessConfig,
																   ILiveLinkDeviceCapability_Ingest* InOwner,
																   FPrivateToken)
{
	TakeId = InTakeId;
	ProcessConfig = InProcessConfig;
	Owner = InOwner;

	NumberOfSteps = UE::CaptureManager::GetNumberOfProcessTasks(ProcessConfig);
}

bool FIngestCapability_ProcessContext::IsDone() const
{
	return ProcessConfig == EIngestCapability_ProcessConfig::None; // ProcessConfig == 0
}

UIngestCapability_ProcessHandle::UIngestCapability_ProcessHandle()
	: Context(nullptr)
{
}

void UIngestCapability_ProcessHandle::Initialize(TUniquePtr<FIngestCapability_ProcessContext> InContext)
{
	Context = MoveTemp(InContext);
}

int32 UIngestCapability_ProcessHandle::GetTakeId() const
{
	check(Context);

	return Context->TakeId;
}

bool UIngestCapability_ProcessHandle::IsDone() const
{
	check(Context);

	return Context->IsDone();
}

FProcessFinishReporter& UIngestCapability_ProcessHandle::OnProcessFinishReporterDynamic()
{
	check(Context);

	return Context->ProcessFinishedReporterDynamic;
}

FIngestProcessFinishReporter& UIngestCapability_ProcessHandle::OnProcessFinishReporter()
{
	check(Context);

	return Context->ProcessFinishedReporter;
}

FProcessProgressReporter& UIngestCapability_ProcessHandle::OnProcessProgressReporterDynamic()
{
	check(Context);

	return Context->ProcessProgressReporterDynamic;
}

FIngestProcessProgressReporter& UIngestCapability_ProcessHandle::OnProcessProgressReporter()
{
	check(Context);

	return Context->ProcessProgressReporter;
}

bool UIngestCapability_ProcessResult::IsValid() const
{
	return Code == 0;
}

bool UIngestCapability_ProcessResult::IsError() const
{
	return !IsValid();
}

UIngestCapability_ProcessResult* UIngestCapability_ProcessResult::Success()
{
	return NewObject<UIngestCapability_ProcessResult>();
}

UIngestCapability_ProcessResult* UIngestCapability_ProcessResult::Error(FText InMessage, int32 InCode)
{
	UIngestCapability_ProcessResult* Result = NewObject<UIngestCapability_ProcessResult>();

	Result->Message = InMessage;
	Result->Code = InCode;
	
	return Result;
}
