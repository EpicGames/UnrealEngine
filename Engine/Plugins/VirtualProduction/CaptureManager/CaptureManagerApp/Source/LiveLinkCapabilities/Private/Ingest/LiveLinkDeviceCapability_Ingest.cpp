// Copyright Epic Games, Inc. All Rights Reserved.

#include "Ingest/LiveLinkDeviceCapability_Ingest.h"

#include "LiveLinkDevice.h"
#include "Widgets/Text/STextBlock.h"

SHeaderRow::FColumn::FArguments& ULiveLinkDeviceCapability_Ingest::GenerateHeaderForColumn(const FName InColumnId, SHeaderRow::FColumn::FArguments& InArgs)
{
	if (InColumnId == Column_IngestSupport)
	{
		return InArgs.DefaultLabel(FText::FromString("Ingest"))
			.DefaultTooltip(FText::FromString("Device supports ingest"))
			.FillSized(30.0f);
	}

	return Super::GenerateHeaderForColumn(InColumnId, InArgs);
}

TSharedPtr<SWidget> ULiveLinkDeviceCapability_Ingest::GenerateWidgetForColumn(const FName InColumnId, const FLiveLinkDeviceWidgetArguments& InArgs, ULiveLinkDevice* InDevice)
{
	return SNew(STextBlock)
		.Text_Lambda(
			[WeakDevice = TWeakObjectPtr<ULiveLinkDevice>(InDevice)]
			()
			{
				if (ULiveLinkDevice* Device = WeakDevice.Get())
				{
					const bool bDeviceSupportsIngest = Device->Implements<ULiveLinkDeviceCapability_Ingest>();
					if (bDeviceSupportsIngest)
					{
						return FText::FromString(TEXT("Y"));
					}
				}

				return FText::FromString(TEXT("N"));
			}
		);
}

ILiveLinkDeviceCapability_Ingest::ILiveLinkDeviceCapability_Ingest()
{
	RegisterEvent(FString(FIngestCapability_TakeAddedEvent::Name));
	RegisterEvent(FString(FIngestCapability_TakeUpdatedEvent::Name));
	RegisterEvent(FString(FIngestCapability_TakeRemovedEvent::Name));
}

UIngestCapability_ProcessHandle* ILiveLinkDeviceCapability_Ingest::CreateIngestProcess_Implementation(int32 InTakeId, EIngestCapability_ProcessConfig InProcessConfig)
{
	TUniquePtr<FIngestCapability_ProcessContext> Context = 
		MakeUnique<FIngestCapability_ProcessContext>(InTakeId, InProcessConfig, this, FIngestCapability_ProcessContext::FPrivateToken());

	TStrongObjectPtr<UIngestCapability_ProcessHandle> IngestProcessHandle(NewObject<UIngestCapability_ProcessHandle>());
	IngestProcessHandle->Initialize(MoveTemp(Context));

	return IngestProcessHandle.Get();
}

void ILiveLinkDeviceCapability_Ingest::RunIngestProcess_Implementation(UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InOptions)
{
	using namespace UE::CaptureManager;

	check(InProcessHandle->Context);

	FIngestCapability_ProcessContext* Context = InProcessHandle->Context.Get();

	if (!Context->IngestOptions)
	{
		Context->IngestOptions = TStrongObjectPtr<const UIngestCapability_Options>(InOptions);
	}

	if (!Context->TaskProgress)
	{
		Context->TaskProgress =
			MakeShared<FTaskProgress>(Context->NumberOfSteps,
									  FTaskProgress::FProgressReporter::CreateLambda([this, InProcessHandle](double InProgress)
									  {
										  ExecuteProcessTotalProgressReporter(InProcessHandle, InProgress);
									  }));
	}

	RunIngestProcess(InProcessHandle);
}

void ILiveLinkDeviceCapability_Ingest::UpdateTakeList_Implementation(UIngestCapability_UpdateTakeListCallback* InCallback)
{
	StopToken = StopRequester.CreateToken();

	RunUpdateTakeList(InCallback);
}

void ILiveLinkDeviceCapability_Ingest::RequestAbortUpdateTakeList_Implementation()
{
	StopRequester.RequestStop();
}

UIngestCapability_TakeInformation* ILiveLinkDeviceCapability_Ingest::GetTakeInformation_Implementation(int32 InTakeId) const
{
	TOptional<FTakeMetadata> TakeMetadataOpt = GetTakeMetadata(InTakeId);

	if (!TakeMetadataOpt.IsSet())
	{
		return nullptr;
	}

	FTakeMetadata TakeMetadata = TakeMetadataOpt.GetValue();

	UIngestCapability_TakeInformation* TakeInformation = 
		NewObject<UIngestCapability_TakeInformation>();

	TakeInformation->SlateName = TakeMetadata.Slate;
	TakeInformation->TakeNumber = TakeMetadata.TakeNumber;

	if (TakeMetadata.DateTime.IsSet())
	{
		TakeInformation->DateTime = TakeMetadata.DateTime.GetValue();
	}

	return TakeInformation;
}

TArray<int32> ILiveLinkDeviceCapability_Ingest::GetTakeIdentifiers_Implementation() const
{
	TArray<int32> TakeIdentifiers;

	FScopeLock Lock(&TakeMetadataMutex);
	TakeMetadataMap.GetKeys(TakeIdentifiers);

	return TakeIdentifiers;
}

TOptional<FTakeMetadata> ILiveLinkDeviceCapability_Ingest::GetTakeMetadata(int32 InTakeId) const
{
	FScopeLock Lock(&TakeMetadataMutex);
	if (const FTakeMetadata* TakeMetadata = TakeMetadataMap.Find(InTakeId))
	{
		return *TakeMetadata;
	}

	return {};
}

void ILiveLinkDeviceCapability_Ingest::ExecuteProcessFinishedReporter(const UIngestCapability_ProcessHandle* InProcessHandle, TValueOrError<void, FIngestCapability_Error> InMaybeError)
{
	UIngestCapability_ProcessResult* Result =
		UIngestCapability_ProcessResult::Success();

	check(InProcessHandle->Context);

	FIngestCapability_ProcessContext* Context = InProcessHandle->Context.Get();

	bool bShouldContinue = true;
	if (InMaybeError.HasError())
	{
		FIngestCapability_Error Error = InMaybeError.GetError();
		Result->Code = static_cast<int32>(Error.GetCode());
		Result->Message = FText::FromString(Error.GetMessage());

		bShouldContinue = false;
	}
	else
	{
		Context->ProcessConfig = Context->ProcessConfig & ~Context->CurrentStep;
		bShouldContinue = !Context->IsDone();
	}

	Context->ProcessFinishedReporterDynamic.ExecuteIfBound(InProcessHandle, Result);
	Context->ProcessFinishedReporter(InProcessHandle, MoveTemp(InMaybeError));

	if (bShouldContinue)
	{
		RunIngestProcess(InProcessHandle);
	}
}

void ILiveLinkDeviceCapability_Ingest::ExecuteProcessProgressReporter(const UIngestCapability_ProcessHandle* InProcessHandle, double InProgress)
{
	check(InProcessHandle->Context);

	FIngestCapability_ProcessContext* Context = InProcessHandle->Context.Get();

	Context->CurrentTask.Update(InProgress);
}

void ILiveLinkDeviceCapability_Ingest::ExecuteProcessTotalProgressReporter(const UIngestCapability_ProcessHandle* InProcessHandle, double InProgress)
{
	check(InProcessHandle->Context);

	FIngestCapability_ProcessContext* Context = InProcessHandle->Context.Get();

	Context->ProcessProgressReporter(InProcessHandle, InProgress);
	Context->ProcessProgressReporterDynamic.ExecuteIfBound(InProcessHandle, InProgress);
}

void ILiveLinkDeviceCapability_Ingest::ExecuteUpdateTakeListCallback(UIngestCapability_UpdateTakeListCallback* InCallback, const TArray<int32>& InTakeIdentifiers)
{
	if (ensure(InCallback))
	{
		InCallback->DynamicCallback.ExecuteIfBound(InTakeIdentifiers);
		InCallback->Callback(InTakeIdentifiers);
	}
}

int32 ILiveLinkDeviceCapability_Ingest::AddTake(FTakeMetadata InTakeMetadata)
{
	int32 NewTakeId = CurrentTakeId.fetch_add(1);

	FScopeLock Lock(&TakeMetadataMutex);
	TakeMetadataMap.Add(NewTakeId, MoveTemp(InTakeMetadata));

	return NewTakeId;
}

void ILiveLinkDeviceCapability_Ingest::RemoveTake(int32 InTakeId)
{
	FScopeLock Lock(&TakeMetadataMutex);
	TakeMetadataMap.Remove(InTakeId);
}

void ILiveLinkDeviceCapability_Ingest::RemoveAllTakes()
{
	FScopeLock Lock(&TakeMetadataMutex);
	TakeMetadataMap.Empty();
}

bool ILiveLinkDeviceCapability_Ingest::UpdateTake(int32 InTakeId, FTakeMetadata InTakeMetadata)
{
	FScopeLock Lock(&TakeMetadataMutex);
	if (FTakeMetadata* TakeMetadata = TakeMetadataMap.Find(InTakeId))
	{
		*TakeMetadata = MoveTemp(InTakeMetadata);
		return true;
	}

	return false;
}

bool ILiveLinkDeviceCapability_Ingest::IsUpdateTakeListAbortRequested() const
{
	return StopToken.IsStopRequested();
}

void ILiveLinkDeviceCapability_Ingest::RunIngestProcess(const UIngestCapability_ProcessHandle* InProcessHandle)
{
	check(InProcessHandle->Context);

	FIngestCapability_ProcessContext* Context = InProcessHandle->Context.Get();

	Context->CurrentTask = Context->TaskProgress->StartTask();

	if (EnumHasAnyFlags(Context->ProcessConfig, EIngestCapability_ProcessConfig::DownloadStep))
	{
		Context->CurrentStep = EIngestCapability_ProcessConfig::DownloadStep;
		RunDownloadTake(InProcessHandle, Context->IngestOptions.Get());
	}
	else if (EnumHasAnyFlags(Context->ProcessConfig, EIngestCapability_ProcessConfig::ConvertAndUploadStep))
	{
		Context->CurrentStep = EIngestCapability_ProcessConfig::ConvertAndUploadStep;
		RunConvertAndUploadTake(InProcessHandle, Context->IngestOptions.Get());
	}
}