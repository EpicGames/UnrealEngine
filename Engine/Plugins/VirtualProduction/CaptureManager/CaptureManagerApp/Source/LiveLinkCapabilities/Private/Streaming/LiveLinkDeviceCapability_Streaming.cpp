// Copyright Epic Games, Inc. All Rights Reserved.

#include "Streaming/LiveLinkDeviceCapability_Streaming.h"

#include "Profile/IMediaProfileManager.h"
#include "Profile/MediaProfile.h"
#include "Profile/MediaProfileSettings.h"
#include "Profile/MediaProfilePlaybackManager.h"

#include "Engine/Engine.h"
#include "LiveLinkDeviceSubsystem.h"

#include "MediaSource.h"

#include "Containers/Ticker.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "LiveLinkDeviceCapability_Streaming"


void ULiveLinkDeviceCapability_Streaming::OnDeviceSubsystemInitialized()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	if (Subsystem)
	{
		Subsystem->OnDeviceAdded().AddUObject(this, &ULiveLinkDeviceCapability_Streaming::OnDeviceAdded);
		Subsystem->OnDeviceRemoved().AddUObject(this, &ULiveLinkDeviceCapability_Streaming::OnDeviceRemoved);
	}
}

void ULiveLinkDeviceCapability_Streaming::OnDeviceSubsystemDeinitializing()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	if (Subsystem)
	{
		Subsystem->OnDeviceAdded().RemoveAll(this);
		Subsystem->OnDeviceRemoved().RemoveAll(this);
	}
}

SHeaderRow::FColumn::FArguments& ULiveLinkDeviceCapability_Streaming::GenerateHeaderForColumn(const FName InColumnId, SHeaderRow::FColumn::FArguments& InArgs)
{
	return Super::GenerateHeaderForColumn(InColumnId, InArgs);
}

TSharedPtr<SWidget> ULiveLinkDeviceCapability_Streaming::GenerateWidgetForColumn(const FName InColumnId, const FLiveLinkDeviceWidgetArguments& InArgs, ULiveLinkDevice* InDevice)
{
	return Super::GenerateWidgetForColumn(InColumnId, InArgs, InDevice);
}

void ULiveLinkDeviceCapability_Streaming::OnDeviceAdded(FGuid InGuid, ULiveLinkDevice* InDevice)
{
	if (InDevice->Implements<ULiveLinkDeviceCapability_Streaming>())
	{
		TScriptInterface<ILiveLinkDeviceCapability_Streaming> Device(InDevice);
		Device->Init();
	}
}

void ULiveLinkDeviceCapability_Streaming::OnDeviceRemoved(FGuid InGuid, ULiveLinkDevice* InDevice)
{
	if (InDevice->Implements<ULiveLinkDeviceCapability_Streaming>())
	{
		TScriptInterface<ILiveLinkDeviceCapability_Streaming> Device(InDevice);
		Device->Deinit();
	}
}

class ILiveLinkDeviceCapability_Streaming::FImpl : public TSharedFromThis<ILiveLinkDeviceCapability_Streaming::FImpl>
{
public:

	FImpl() = default;

	void Init();
	void Deinit();

	UMediaSource* GetMediaSource() const;
	void SetMediaSource(UMediaSource* InSource);
	void SetMediaSourceName(const FString& InName);
	void RegisterMediaSource(UMediaSource* InMediaSource);
	void UnregisterMediaSource();
	bool IsStarted() const { return bIsStarted; }

private:

	void OnMediaProfileChanged(UMediaProfile* InPrevious, UMediaProfile* InNew);

	void UpdateLabelForMediaSource_GameThread(const FString& InName, UMediaProfile* InMediaProfile);
	void RegisterMediaSource_GameThread(UMediaSource* InMediaSource, UMediaProfile* InMediaProfile);
	void UnregisterMediaSource_GameThread(UMediaProfile* InMediaProfile);

	void RemoveCurrentMediaSource(UMediaProfile* InMediaProfile);

	mutable FCriticalSection Mutex;
	TStrongObjectPtr<UMediaSource> CurrentMediaSource = nullptr;

	// Updated only from GameThread
	FString MediaSourceName;
	std::atomic_bool bIsStarted = false;

	FDelegateHandle DelegateHandle;
};

void ILiveLinkDeviceCapability_Streaming::FImpl::Init()
{
	IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
	DelegateHandle = MediaProfileManager.OnMediaProfileChanged().AddSP(this, &ILiveLinkDeviceCapability_Streaming::FImpl::OnMediaProfileChanged);
}

void ILiveLinkDeviceCapability_Streaming::FImpl::Deinit()
{
	IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
	MediaProfileManager.OnMediaProfileChanged().Remove(DelegateHandle);
}

UMediaSource* ILiveLinkDeviceCapability_Streaming::FImpl::GetMediaSource() const
{
	FScopeLock Lock(&Mutex);
	return CurrentMediaSource.Get();
}

void ILiveLinkDeviceCapability_Streaming::FImpl::SetMediaSource(UMediaSource* InSource)
{
	FScopeLock Lock(&Mutex);
	CurrentMediaSource.Reset(InSource);
}

void ILiveLinkDeviceCapability_Streaming::FImpl::SetMediaSourceName(const FString& InName)
{
	if (IsInGameThread())
	{
		MediaSourceName = InName;

		if (IsStarted())
		{
			IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
			UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

			UpdateLabelForMediaSource_GameThread(MediaSourceName, MediaProfile);
		}
	}
	else
	{
		ExecuteOnGameThread(TEXT("Updating Media Profile"), [This = AsWeak(), Name = InName]() mutable
		{
			if (TSharedPtr<FImpl> SharedThis = This.Pin())
			{
				SharedThis->MediaSourceName = MoveTemp(Name);

				if (SharedThis->IsStarted())
				{
					IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
					UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

					SharedThis->UpdateLabelForMediaSource_GameThread(SharedThis->MediaSourceName, MediaProfile);
				}
			}
		});
	}
}

void ILiveLinkDeviceCapability_Streaming::FImpl::RegisterMediaSource(UMediaSource* InMediaSource)
{
	TStrongObjectPtr<UMediaSource> MediaSource(InMediaSource);

	if (IsInGameThread())
	{
		IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
		UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

		RegisterMediaSource_GameThread(MediaSource.Get(), MediaProfile);
		UpdateLabelForMediaSource_GameThread(MediaSourceName, MediaProfile);
		bIsStarted = true;
	}
	else
	{
		ExecuteOnGameThread(TEXT("Updating Media Profile"), [This = AsWeak(), MediaSource = MoveTemp(MediaSource)]() mutable
		{
			if (TSharedPtr<FImpl> SharedThis = This.Pin())
			{
				IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
				UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

				SharedThis->RegisterMediaSource_GameThread(MediaSource.Get(), MediaProfile);
				SharedThis->UpdateLabelForMediaSource_GameThread(SharedThis->MediaSourceName, MediaProfile);
				SharedThis->bIsStarted = true;
			}
		});
	}
}

void ILiveLinkDeviceCapability_Streaming::FImpl::UnregisterMediaSource()
{
	if (IsInGameThread())
	{
		IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
		UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

		UnregisterMediaSource_GameThread(MediaProfile);
		bIsStarted = false;
	}
	else
	{
		ExecuteOnGameThread(TEXT("Updating Media Profile"), [This = AsWeak()]() mutable
		{
			if (TSharedPtr<FImpl> SharedThis = This.Pin())
			{
				IMediaProfileManager& MediaProfileManager = IMediaProfileManager::Get();
				UMediaProfile* MediaProfile = MediaProfileManager.GetCurrentMediaProfile();

				SharedThis->UnregisterMediaSource_GameThread(MediaProfile);
				SharedThis->bIsStarted = false;
			}
		});
	}
}

void ILiveLinkDeviceCapability_Streaming::FImpl::OnMediaProfileChanged(UMediaProfile* InPrevious, UMediaProfile* InNew)
{
	FScopeLock Lock(&Mutex);

	if (!CurrentMediaSource.Get())
	{
		return;
	}

	if (!IsStarted())
	{
		return;
	}

	if (InPrevious)
	{
		UnregisterMediaSource_GameThread(InPrevious);
	}
	
	if (InNew)
	{
		RegisterMediaSource_GameThread(CurrentMediaSource.Get(), InNew);
		UpdateLabelForMediaSource_GameThread(MediaSourceName, InNew);
	}
}

void ILiveLinkDeviceCapability_Streaming::FImpl::UpdateLabelForMediaSource_GameThread(const FString& InName, UMediaProfile* InMediaProfile)
{
	if (!InMediaProfile)
	{
		return;
	}

#if WITH_EDITOR
	int32 Index = InMediaProfile->FindMediaSourceIndex(GetMediaSource());
	if (Index == INDEX_NONE)
	{
		return;
	}

	if (InMediaProfile->GetLabelForMediaSource(Index) == InName)
	{
		return;
	}

	FScopedTransaction UpdateMediaProfileTransaction(LOCTEXT("UpdateMediaProfileTransaction", "Update Media Profile"));
	InMediaProfile->Modify();

	// Will publish the property changed event
	InMediaProfile->SetLabelForMediaSource(Index, InName);

	InMediaProfile->MarkPackageDirty();
#endif
}

void ILiveLinkDeviceCapability_Streaming::FImpl::RegisterMediaSource_GameThread(UMediaSource* InMediaSource, UMediaProfile* InMediaProfile)
{
	if (!InMediaProfile)
	{
		return;
	}

	FScopedTransaction UpdateMediaProfileTransaction(LOCTEXT("UpdateMediaProfileTransaction", "Update Media Profile"));
	InMediaProfile->Modify();

	RemoveCurrentMediaSource(InMediaProfile);

	if (InMediaSource)
	{
		InMediaProfile->AddMediaSource(InMediaSource);
	}

	int32 NumOfSources = InMediaProfile->NumMediaSources();
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(NumOfSources, true);

#if WITH_EDITOR
	FProperty* Property = FindFProperty<FProperty>(UMediaProfile::StaticClass(), "MediaSources"); // MediaSources is protected field

	if (Property)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayAdd);
		InMediaProfile->PostEditChangeProperty(PropertyChangedEvent);
	}
#endif

	InMediaProfile->MarkPackageDirty();
}

void ILiveLinkDeviceCapability_Streaming::FImpl::UnregisterMediaSource_GameThread(UMediaProfile* InMediaProfile)
{
	if (!InMediaProfile)
	{
		return;
	}

	FScopedTransaction RemoveSourceMediaProfileTransaction(LOCTEXT("RemoveSourceMediaProfileTransaction", "Remove Media Source"));
	InMediaProfile->Modify();

	{
		// Close the source if we are unregistering
		FScopeLock Lock(&Mutex);

		UMediaProfilePlaybackManager::FCloseSourceArgs Args;
		Args.bForceClose = true;
		InMediaProfile->GetPlaybackManager()->CloseSource(CurrentMediaSource.Get(), Args);
	}

	RemoveCurrentMediaSource(InMediaProfile);

	int32 NumOfSources = InMediaProfile->NumMediaSources();
	GetMutableDefault<UMediaProfileSettings>()->FillDefaultMediaSourceProxies(NumOfSources, true);

#if WITH_EDITOR
	FProperty* Property = FindFProperty<FProperty>(UMediaProfile::StaticClass(), "MediaSources"); // MediaSources is protected field

	if (Property)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayRemove);
		InMediaProfile->PostEditChangeProperty(PropertyChangedEvent);
	}
#endif

	InMediaProfile->MarkPackageDirty();
}

void ILiveLinkDeviceCapability_Streaming::FImpl::RemoveCurrentMediaSource(UMediaProfile* InMediaProfile)
{
	FScopeLock Lock(&Mutex);

	int32 Index = InMediaProfile->FindMediaSourceIndex(CurrentMediaSource.Get());
	if (Index != INDEX_NONE)
	{
		InMediaProfile->RemoveMediaSource(Index);
	}
}

ILiveLinkDeviceCapability_Streaming::ILiveLinkDeviceCapability_Streaming()
	: Impl(MakeShared<FImpl>())
{
}

void ILiveLinkDeviceCapability_Streaming::StartStreaming_Implementation()
{
	Impl->RegisterMediaSource(Impl->GetMediaSource());
}

void ILiveLinkDeviceCapability_Streaming::StopStreaming_Implementation()
{
	Impl->UnregisterMediaSource();
}

bool ILiveLinkDeviceCapability_Streaming::IsStreaming_Implementation() const
{
	return Impl->IsStarted();
}

UMediaSource* ILiveLinkDeviceCapability_Streaming::GetMediaSource_Implementation() const
{
	return Impl->GetMediaSource();
}

void ILiveLinkDeviceCapability_Streaming::SetMediaSourceName(const FString& InName)
{
	Impl->SetMediaSourceName(InName);
}

void ILiveLinkDeviceCapability_Streaming::SetMediaSource(UMediaSource* InMediaSource)
{
	Impl->SetMediaSource(InMediaSource);
}

void ILiveLinkDeviceCapability_Streaming::DeleteMediaSource()
{
	Impl->SetMediaSource(nullptr);
}

void ILiveLinkDeviceCapability_Streaming::Init()
{
	Impl->Init();
}

void ILiveLinkDeviceCapability_Streaming::Deinit()
{
	Impl->Deinit();
}

#undef LOCTEXT_NAMESPACE
