// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Event.h"
#include "Async/EventSourceUtils.h"

#include "Ingest/IngestCapability_TakeInformation.h"

struct FIngestCapability_TakeBaseEvent : public UE::CaptureManager::FCaptureEvent
{
	FIngestCapability_TakeBaseEvent(int32 InTakeId, FString InName)
		: FCaptureEvent(MoveTemp(InName))
		, TakeId(InTakeId)
	{
	}

	int32 TakeId;
};

struct LIVELINKCAPABILITIES_API FIngestCapability_TakeAddedEvent : public FIngestCapability_TakeBaseEvent
{
	static constexpr FStringView Name = TEXT("IngestCapability_TakeAdded");

	explicit FIngestCapability_TakeAddedEvent(int32 InTakeId)
		: FIngestCapability_TakeBaseEvent(InTakeId, FString(Name))
	{
	}
};

struct LIVELINKCAPABILITIES_API FIngestCapability_TakeUpdatedEvent : public FIngestCapability_TakeBaseEvent
{
	static constexpr FStringView Name = TEXT("IngestCapability_TakeUpdated");

	explicit FIngestCapability_TakeUpdatedEvent(int32 InTakeId)
		: FIngestCapability_TakeBaseEvent(InTakeId, FString(Name))
	{
	}
};

struct LIVELINKCAPABILITIES_API FIngestCapability_TakeRemovedEvent : public FIngestCapability_TakeBaseEvent
{
	static constexpr FStringView Name = TEXT("IngestCapability_TakeRemoved");

	explicit FIngestCapability_TakeRemovedEvent(int32 InTakeId)
		: FIngestCapability_TakeBaseEvent(InTakeId, FString(Name))
	{
	}
};

struct LIVELINKCAPABILITIES_API FIngestCapability_TakeListResetEvent : public UE::CaptureManager::FCaptureEvent
{
	static constexpr FStringView Name = TEXT("IngestCapability_TakeListReset");

	explicit FIngestCapability_TakeListResetEvent(FString InName)
		: FCaptureEvent(MoveTemp(InName))
	{
	}
};