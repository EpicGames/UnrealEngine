// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "CaptureManagerConversionTypes.h"

#include "IngestCapability_Options.generated.h"

// Deprecated type aliases for backward compatibility.
using EIngestCapability_ImagePixelFormat UE_DEPRECATED(5.8, "Use ECaptureManagerPixelFormat instead.") = ECaptureManagerPixelFormat;
using EIngestCapability_ImageRotation UE_DEPRECATED(5.8, "Use ECaptureManagerRotation instead.") = ECaptureManagerRotation;

USTRUCT(BlueprintType)
struct LIVELINKCAPABILITIES_API FIngestCapability_VideoOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString FileNamePrefix;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString Format;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	ECaptureManagerPixelFormat PixelFormat = ECaptureManagerPixelFormat::U8_BGRA;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	ECaptureManagerRotation Rotation = ECaptureManagerRotation::None;
};

USTRUCT(BlueprintType)
struct LIVELINKCAPABILITIES_API FIngestCapability_AudioOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString FileNamePrefix;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString Format;
};

UCLASS(BlueprintType)
class LIVELINKCAPABILITIES_API UIngestCapability_Options : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString WorkingDirectory;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString DownloadDirectory;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FIngestCapability_VideoOptions Video;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FIngestCapability_AudioOptions Audio;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Live Link Device|Ingest")
	FString UploadHostName;
};
