// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "Misc/DateTime.h"
#include "CaptureManagerTakeId.h"

#include "IngestCapability_TakeInformation.generated.h"

UCLASS(BlueprintType)
class LIVELINKCAPABILITIES_API UIngestCapability_TakeInformation 
	: public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = TakeInfo)
	FString DeviceName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = TakeInfo)
	FString SlateName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = TakeInfo)
	int32 TakeNumber = -1;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = TakeInfo)
	FDateTime DateTime;

	UFUNCTION(BlueprintCallable, Category = TakeInfo)
	FString GetDateTimeString() const
	{
		return DateTime.ToIso8601();
	}
};

