// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"

#include "Async/ManagedDelegate.h"

#include "Ingest/IngestCapability_TakeInformation.h"

#include "IngestCapability_UpdateTakeList.generated.h"

UDELEGATE()
DECLARE_DYNAMIC_DELEGATE_OneParam(FUpdateTakeListCallback, TArray<int32>, TakeIdentifiers);

using FIngestUpdateTakeListCallback = UE::CaptureManager::TManagedDelegate<TArray<int32>>;

UCLASS()
class LIVELINKCAPABILITIES_API UIngestCapability_UpdateTakeListCallback : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Live Link Device|Ingest")
	FUpdateTakeListCallback DynamicCallback;

	FIngestUpdateTakeListCallback Callback;
};
