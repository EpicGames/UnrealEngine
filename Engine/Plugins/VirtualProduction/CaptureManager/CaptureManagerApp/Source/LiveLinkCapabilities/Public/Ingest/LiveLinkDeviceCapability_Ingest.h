// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDeviceCapability.h"

#include "Templates/ValueOrError.h"
#include "CaptureManagerTakeMetadata.h"

#include "Async/StopToken.h"

#include "Ingest/IngestCapability_Options.h"
#include "Ingest/IngestCapability_ProcessHandle.h"
#include "Ingest/IngestCapability_UpdateTakeList.h"
#include "Ingest/IngestCapability_TakeInformation.h"
#include "Ingest/IngestCapability_Events.h"

#include "LiveLinkDeviceCapability_Ingest.generated.h"

UINTERFACE(Blueprintable)
class LIVELINKCAPABILITIES_API ULiveLinkDeviceCapability_Ingest : 
	public ULiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	const FName Column_IngestSupport;

	ULiveLinkDeviceCapability_Ingest()
		: Column_IngestSupport(RegisterTableColumn("IngestSupport"))
	{
	}

	virtual SHeaderRow::FColumn::FArguments& GenerateHeaderForColumn(const FName InColumnId, SHeaderRow::FColumn::FArguments& InArgs) override;
	virtual TSharedPtr<SWidget> GenerateWidgetForColumn(const FName InColumnId, const FLiveLinkDeviceWidgetArguments& InArgs, ULiveLinkDevice* InDevice) override;
};

class LIVELINKCAPABILITIES_API ILiveLinkDeviceCapability_Ingest 
	: public ILiveLinkDeviceCapability
#if CPP
	, public UE::CaptureManager::FCaptureEventSource
#endif
{
	GENERATED_BODY()

public:

	ILiveLinkDeviceCapability_Ingest();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	UIngestCapability_ProcessHandle* CreateIngestProcess(int32 InTakeId, EIngestCapability_ProcessConfig InProcessConfig);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	void RunIngestProcess(UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InOptions);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	void CancelIngestProcess(const UIngestCapability_ProcessHandle* InProcessHandle);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	void UpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback);

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	void RequestAbortUpdateTakeList();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	UIngestCapability_TakeInformation* GetTakeInformation(int32 InTakeId) const;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Ingest")
	TArray<int32> GetTakeIdentifiers() const;

	UIngestCapability_ProcessHandle* CreateIngestProcess_Implementation(int32 InTakeId, EIngestCapability_ProcessConfig InProcessConfig);
	void RunIngestProcess_Implementation(UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InOptions);
	virtual void CancelIngestProcess_Implementation(const UIngestCapability_ProcessHandle* InProcessHandle) PURE_VIRTUAL(ILiveLinkDeviceCapability_Ingest::CancelIngestProcess_Implementation);

	void UpdateTakeList_Implementation(UIngestCapability_UpdateTakeListCallback* InCallback);
	void RequestAbortUpdateTakeList_Implementation();

	UIngestCapability_TakeInformation* GetTakeInformation_Implementation(int32 InTakeId) const;
	TArray<int32> GetTakeIdentifiers_Implementation() const;

	TOptional<FTakeMetadata> GetTakeMetadata(int32 InTakeId) const;

protected:

	void ExecuteProcessFinishedReporter(const UIngestCapability_ProcessHandle* InProcessHandle, TValueOrError<void, FIngestCapability_Error> InMaybeError);
	void ExecuteProcessProgressReporter(const UIngestCapability_ProcessHandle* InProcessHandle, double InProgress);
	void ExecuteUpdateTakeListCallback(UIngestCapability_UpdateTakeListCallback* InCallback, const TArray<int32>& InTakeIdentifiers);
	
	int32 AddTake(FTakeMetadata InTakeMetadata);
	void RemoveTake(int32 InTakeId);
	void RemoveAllTakes();
	bool UpdateTake(int32 InTakeId, FTakeMetadata InTakeMetadata);
	bool IsUpdateTakeListAbortRequested() const;

private:

	void ExecuteProcessTotalProgressReporter(const UIngestCapability_ProcessHandle* InProcessHandle, double InProgress);

	void RunIngestProcess(const UIngestCapability_ProcessHandle* InProcessHandle);

	virtual void RunDownloadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) = 0;
	virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) = 0;
	virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) = 0;

	UE::CaptureManager::FStopRequester StopRequester;
	UE::CaptureManager::FStopToken StopToken;

	mutable FCriticalSection TakeMetadataMutex;
	TMap<int32, FTakeMetadata> TakeMetadataMap;

	std::atomic_int32_t CurrentTakeId = 0;

	friend class UIngestCapability_ProcessHandle;
};