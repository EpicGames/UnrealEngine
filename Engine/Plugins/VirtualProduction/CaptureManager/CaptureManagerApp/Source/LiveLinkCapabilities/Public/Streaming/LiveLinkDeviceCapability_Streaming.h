// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDeviceCapability.h"

#include "LiveLinkDevice.h"

#include "LiveLinkDeviceCapability_Streaming.generated.h"

UINTERFACE(Blueprintable)
class LIVELINKCAPABILITIES_API ULiveLinkDeviceCapability_Streaming :
	public ULiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	
	ULiveLinkDeviceCapability_Streaming() = default;

	/** Called at completion of Live Link device subsystem initialization. */
	virtual void OnDeviceSubsystemInitialized();

	/** Called at the beginning of Live Link device subsystem de-initialization. */
	virtual void OnDeviceSubsystemDeinitializing();

	virtual SHeaderRow::FColumn::FArguments& GenerateHeaderForColumn(const FName InColumnId, SHeaderRow::FColumn::FArguments& InArgs) override;
	virtual TSharedPtr<SWidget> GenerateWidgetForColumn(const FName InColumnId, const FLiveLinkDeviceWidgetArguments& InArgs, ULiveLinkDevice* InDevice) override;

private:

	void OnDeviceAdded(FGuid InGuid, ULiveLinkDevice* InDevice);
	void OnDeviceRemoved(FGuid InGuid, ULiveLinkDevice* InDevice);
};

class LIVELINKCAPABILITIES_API ILiveLinkDeviceCapability_Streaming
	: public ILiveLinkDeviceCapability
{
	GENERATED_BODY()

public:

	ILiveLinkDeviceCapability_Streaming();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Streaming")
	void StartStreaming();
	void StartStreaming_Implementation();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Streaming")
	void StopStreaming();
	void StopStreaming_Implementation();

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Streaming")
	bool IsStreaming() const;
	bool IsStreaming_Implementation() const;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Live Link Device|Streaming")
	class UMediaSource* GetMediaSource() const;
	class UMediaSource* GetMediaSource_Implementation() const;

protected:

	void SetMediaSourceName(const FString& InName);
	void SetMediaSource(class UMediaSource* InMediaSource);
	void DeleteMediaSource();

private:

	void Init();
	void Deinit();

	friend class ULiveLinkDeviceCapability_Streaming;

	class FImpl;
	TSharedPtr<FImpl> Impl;
};